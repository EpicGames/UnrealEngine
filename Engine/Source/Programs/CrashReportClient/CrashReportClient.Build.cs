// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CrashReportClient : ModuleRules
{
	public CrashReportClient(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"CrashReportCore",
				"HTTP",
				"Json",
				"Projects",
				"PakFile",
				"XmlParser",
				"Analytics",
				"AnalyticsET",
				"DesktopPlatform",
				"LauncherPlatform",
				"InputCore",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"MessageLog",
			}
		);

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
				"SlateReflector",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
				"SlateReflector",
				}
			);
		}

		PrivateDefinitions.AddRange(
			new string[]
			{
				"CRASH_REPORT_WITH_MTBF=1",
			}
		);
	}
}
