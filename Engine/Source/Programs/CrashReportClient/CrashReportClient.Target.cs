// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("Win64", "Mac", "Linux", "LinuxArm64")]
[SupportedConfigurations(UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Shipping)]
public class CrashReportClientTarget : TargetRules
{
	// Default crash reporting url. This is the default value. Overrideable per project.
	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientBuildSettings", "DefaultUrl")]
	public string DefaultUrl;
		
	// Default company name displayed in the UX. Overrideable per project
	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientBuildSettings", "DefaultCompanyName")]
	public string DefaultCompanyName;
	
	// Url for telemetry. (Not used by licensees)
	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientBuildSettings", "TelemetryUrl")]
	public string TelemetryUrl;

	// Key used by telemetry (Not used by licensees)
	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientBuildSettings", "TelemetryKey")]
	public string TelemetryKey;

	public CrashReportClientTarget(TargetInfo Target) : this(Target, true)
	{}

	protected CrashReportClientTarget(TargetInfo Target, bool bSetConfiguredDefinitions) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		UndecoratedConfiguration = UnrealTargetConfiguration.Shipping;

		LaunchModuleName = "CrashReportClient";

		if (bBuildEditor == true && Target.Platform != UnrealTargetPlatform.Linux)
		{
			ExtraModuleNames.Add("EditorStyle");
		}

		bLegalToDistributeBinary = true;

		bBuildDeveloperTools = false;

		// CrashReportClient doesn't ever compile with the engine linked in
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bUseLoggingInShipping = true;

		// CrashReportClient.exe has no exports, so no need to verify that a .lib and .exp file was emitted by
		// the linker.
		bHasExports = false;

		bUseChecksInShipping = true;

		// Epic Games Launcher needs to run on OS X 10.9, so CrashReportClient needs this as well
		bEnableOSX109Support = true;

		// Need to disable the bundled version of dbghelp so that CrashDebugHelper can load dbgeng.dll.
		WindowsPlatform.bUseBundledDbgHelp = false;

		// Set the maximum number of cores used
		GlobalDefinitions.Add("UE_TASKGRAPH_THREAD_LIMIT=5");

		GlobalDefinitions.Add("NOINITCRASHREPORTER=1");

		// Since we can't use virtual calls in the constructor allow, any inheriting type to opt out
		// of setting these configured definitions
		if (bSetConfiguredDefinitions)
		{
			GlobalDefinitions.AddRange(SetupConfiguredDefines(
				 DefaultUrl, DefaultCompanyName, TelemetryUrl, TelemetryKey));
		}
	}

	protected static List<string> SetupConfiguredDefines(
		string DefaultUrl, 
		string DefaultCompanyName, 
		string TelemetryUrl, 
		string TelemetryKey)
	{
		var Definitions = new List<string>();
		if (!string.IsNullOrEmpty(DefaultUrl))
		{
			Definitions.Add($"CRC_DEFAULT_URL=\"{DefaultUrl}\"");
		}

		if (!string.IsNullOrEmpty(DefaultCompanyName))
		{
			Definitions.Add($"CRC_DEFAULT_COMPANY_NAME=\"{DefaultCompanyName}\"");
		}

		if(!string.IsNullOrWhiteSpace(TelemetryUrl))
		{
			Definitions.Add($"CRC_TELEMETRY_URL=\"{TelemetryUrl}\"");
		}

		if (!string.IsNullOrWhiteSpace(TelemetryKey))
		{
			Definitions.Add($"CRC_TELEMETRY_KEY=\"{TelemetryKey}\"");
		}

		return Definitions;
	}
}
