// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CrashReportClientEditor : CrashReportClient
{
	public CrashReportClientEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		// Deactivated in 4.25.1: it is suspected to be responsible for crashes in CRC.
		bool bHostRecoverySvc = false;

		PrivateDefinitions.AddRange(
			new string[]
			{
				"CRASH_REPORT_WITH_MTBF=1",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"EditorAnalyticsSession",
			}
		);

		if (bHostRecoverySvc)
		{
			PrivateDefinitions.AddRange(
				new string[]
				{
					"CRASH_REPORT_WITH_RECOVERY=1",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Concert",
					"Messaging",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"ConcertSyncCore",
					"ConcertSyncServer",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"ConcertSyncServer",
					"UdpMessaging",
				}
			);
		}
	}
}
