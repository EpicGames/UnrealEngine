// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Win64", "Mac", "Linux")]
[SupportedConfigurations(UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Shipping)]
public sealed class CrashReportClientEditorTarget : CrashReportClientTarget
{
	// Override the configuration values from CrashReportClient with these using another
	// configuration block: [CrashReportClientEditorBuildSettings]. See CrashReportClient.target.cs for
	// descriptions of the settings.
	
	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientEditorBuildSettings", "DefaultUrl")]
	public new string DefaultUrl;
		
	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientEditorBuildSettings", "DefaultCompanyName")]
	public new string DefaultCompanyName;
	
	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientEditorBuildSettings", "TelemetryUrl")]
	public new string TelemetryUrl;

	[ConfigFile(ConfigHierarchyType.Engine, "CrashReportClientEditorBuildSettings", "TelemetryKey")]
	public new string TelemetryKey;
	
	public CrashReportClientEditorTarget(TargetInfo Target) : base(Target, false /* bSetConfiguredDefinitions */)
	{
		LaunchModuleName = "CrashReportClientEditor";

		// Disabled in 4.25.1 because it is suspected to cause unexpected crash.
		bool bHostRecoverySvc = false;

		bBuildWithEditorOnlyData = false;
		bBuildDeveloperTools = true;

		if (bHostRecoverySvc)
		{
			AdditionalPlugins.Add("UdpMessaging");
			AdditionalPlugins.Add("ConcertSyncServer");
			bCompileWithPluginSupport = true; // Enable Developer plugins (like Concert!)

			if (Target.Configuration == UnrealTargetConfiguration.Shipping && LinkType == TargetLinkType.Monolithic)
			{
				// DisasterRecovery/Concert needs message bus to run. If not enabled, Recovery Service will self-disable as well. In Shipping
				// message bus is turned off by default but for a monolithic build, it can be turned on just for this executable.
				GlobalDefinitions.Add("PLATFORM_SUPPORTS_MESSAGEBUS=1");
			}
		}
		
		// We can now set the configured definitions from CrashReportClientEditorBuildSettings section
		GlobalDefinitions.AddRange(SetupConfiguredDefines(
			DefaultUrl, DefaultCompanyName, TelemetryUrl, TelemetryKey));
	}
	
}
