// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportAnalyticsSessionSummary.h"
#include "AnalyticsPropertyStore.h"
#include "AnalyticsSessionSummaryManager.h"
#include "AnalyticsSessionSummarySender.h"
#include "IAnalyticsProviderET.h"
#include "CrashReportAnalytics.h"
#include "Containers/Map.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "Internationalization/Regex.h"
#include "Logging/LogMacros.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/CoreDelegates.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Templates/UnrealTemplate.h"

DEFINE_LOG_CATEGORY_STATIC(LogCrashReportClientDiagnostics, Log, All)

#if PLATFORM_WINDOWS

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows.h"

/** Handle windows messages. */
LRESULT CALLBACK CrashReportAnalyticsSessionSummaryWindowProc(HWND Hwnd, UINT Msg, WPARAM wParam, LPARAM lParam)
{
	// wParam is true if the user session is going away (and CRC is going to die)
	if (Msg == WM_ENDSESSION && wParam == TRUE)
	{
		FCrashReportAnalyticsSessionSummary::Get().OnUserLoggingOut();
	}
	else if (Msg == WM_CLOSE)
	{
		FCrashReportAnalyticsSessionSummary::Get().OnQuitSignal();
	}
	return DefWindowProc(Hwnd, Msg, wParam, lParam);
}

FProcHandle OpenProcessForMonitoring(uint32 pid)
{
	// Until a crash occurs, for security reasons, restrict CRC accesses on the remote process.
	return FProcHandle(::OpenProcess(PROCESS_DUP_HANDLE | PROCESS_QUERY_LIMITED_INFORMATION | PROCESS_TERMINATE | SYNCHRONIZE, 0, pid));
}

#include "Windows/HideWindowsPlatformTypes.h"

#else // PLATFORM_WINDOWS

FProcHandle OpenProcessForMonitoring(uint32 pid)
{
	return FPlatformProcess::OpenProcess(pid);
}

#endif // PLATFORM_WINDOWS

namespace CrcAnalyticsProperties
{
	// NOTE: Update this when you add/remove/change key behavior. That's useful to track how one changes affects metrics in-dev where users don't always have an engine versions.
	//     - V3 -> Windows optimization for stall/ensure -> The engine only captures the responsible thread so CRC walks 1 thread rather than all threads.
	//     - V4 -> Stripped ensure callstack from the ensure error message to remove noise in the diagnostic log.
	//     - V5 -> Measured time to stack-walk, gather files and addded stall count.
	//     - V6 -> Added MonitorTickCount, MonitorQueryingPipe and App/Death log.
	//     - V7 -> Removed MonitorQueryingPipe. It was added to detect if CRC crashed while reading the pipe. Data showed that wasn't the case.
	constexpr uint32 CrcAnalyticsSummaryVersion = 7;

	/** The exit code of the monitored application. */
	static const TAnalyticsProperty<int32> MonitoredAppExitCode(TEXT("ExitCode"));
	/** Track when CRC detected the death of the monitored app. */
	static const TAnalyticsProperty<FDateTime> MonitoredAppDeathTimestamp(TEXT("DeathTimestamp"));

	/** CRC engine version. In-dev, people don't always recompile CRC and we get disparity between the monitored app and CRC version. */
	static const TAnalyticsProperty<FString> EngineVersion(TEXT("MonitorEngineVersion"));
	/** The version number of the key/set used by CRC. */
	static const TAnalyticsProperty<uint32> SummaryVersionNumber(TEXT("MonitorSummaryVersion"));
	/** The CRC startup timestamp. */
	static const TAnalyticsProperty<FDateTime> StartupTimestamp(TEXT("MonitorStartupTimestamp"));
	/** The CRC timestamp. */
	static const TAnalyticsProperty<FDateTime> Timestamp(TEXT("MonitorTimestamp"));
	/** Number of time CRC analytic thread ticked. */
	static const TAnalyticsProperty<uint32> TickCount(TEXT("MonitorTickCount"));
	/** If CRC raised an exception that was captured by SEH, this is the exception code. */
	static const TAnalyticsProperty<int32> ExceptCode(TEXT("MonitorExceptCode"));
	/** If CRC is about to close because the system sent a quit signal. */
	static const TAnalyticsProperty<bool> QuitSignalRecv(TEXT("MonitorQuitSignalRecv"));
	/** The CRC diagnostic logs. */
	static const TAnalyticsProperty<FString> DiagnosticLogs(TEXT("MonitorLog"));
	/** The CRC session duration in seconds. */
	static const TAnalyticsProperty<int32> SessionDurationSecs(TEXT("MonitorSessionDuration"));
	/** The battery level, if known. */
	static const TAnalyticsProperty<uint32> BatteryLevel(TEXT("MonitorBatteryLevel"));
	/** If the system is connected to AC, if know */
	static const TAnalyticsProperty<bool> IsOnACPower(TEXT("MonitorOnACPower"));
	/** Whether CRC is reporting a crash. */
	static const TAnalyticsProperty<bool> IsReportingCrash(TEXT("MonitorIsReportingCrash"));
	/** Whether CRC is collecting crash artifacts. */
	static const TAnalyticsProperty<bool> IsCollectingCrash(TEXT("MonitorIsCollectingCrash"));
	/** Whether CRC is processing a crash. */
	static const TAnalyticsProperty<bool> IsProcessingCrash(TEXT("MonitorIsProcessingCrash"));
	/** If CRC is about to be killed because the user is logging out (system shutdown/reboot). */
	static const TAnalyticsProperty<bool> UserIsLoggingOut(TEXT("MonitorLoggingOut"));
	/** Whether CRC crashed. */
	static const TAnalyticsProperty<bool> IsCrashing(TEXT("MonitorCrashed"));
	/** Whether CRC was shutdown. */
	static const TAnalyticsProperty<bool> WasShutdown(TEXT("MonitorWasShutdown"));
	/** Number of crash event passed to CRC. (Ensure, Assert, Crash, etc). */
	static const TAnalyticsProperty<uint32> ReportCount(TEXT("MonitorReportCount"));
	/** Number of ensures handled by CRC. */
	static const TAnalyticsProperty<uint32> EnsureCount(TEXT("MonitorEnsureCount"));
	/** Number of assert handled by CRC.*/
	static const TAnalyticsProperty<uint32> AssertCount(TEXT("MonitorAssertCount"));
	/** Number of stalls handed by CRC. */
	static const TAnalyticsProperty<uint32> StallCount(TEXT("MonitorStallCount"));
	/** The worst unattended report time measured (if any). The user is not involved, so it measure how fast CRC can process a crash, especially ensures and stalls. */
	static const TAnalyticsProperty<float> LonguestUnattendedReportSecs(TEXT("MonitorLongestUnattendedReportSecs"));
}

namespace CrashReportClientUtils
{
	/** Flush the store peridically. */
	static const FTimespan PropertyStoreFlushPeriod = FTimespan::FromSeconds(10);

	/** Tick periodically to poll new information. */
	static const FTimespan TickPeriod = FTimespan::FromSeconds(0.5);

	/** Maximum length of the diagnostic log. */
	static constexpr int32 MaxDiagnosticLogLen = 8 * 1024;

#if PLATFORM_WINDOWS && CRASH_REPORT_WITH_MTBF
	HWND Hwnd = NULL;

	/** Create a hidden Windows to intercept WM_ messages, especially the WM_ENDSESSION. */
	void InitPlatformSpecific()
	{
		// Register the window class.
		const wchar_t CLASS_NAME[] = L"CRC Analytics Session Window Message Interceptor";

		WNDCLASS wc = { };
		wc.lpfnWndProc = CrashReportAnalyticsSessionSummaryWindowProc;
		wc.hInstance = hInstance;
		wc.lpszClassName = CLASS_NAME;

		RegisterClass(&wc);

		// Create a window to capture WM_ENDSESSION message (so that we can detect when CRC fails because the user is logging off/shutting down/restarting)
		Hwnd = CreateWindowEx(
			0,                              // Optional window styles.
			CLASS_NAME,                     // Window class
			L"CRC Message Loop Wnd",        // Window text
			WS_OVERLAPPEDWINDOW,            // Window style
			CW_USEDEFAULT, CW_USEDEFAULT, CW_USEDEFAULT, CW_USEDEFAULT, // Size and position
			NULL,         // Parent window
			NULL,         // Menu
			hInstance,    // Instance handle
			NULL          // Additional application data
		);
	}

	/** Pump the message from the hidden Windows. */
	void TickPlatformSpecific()
	{
		if (Hwnd != NULL)
		{
			// Pump the messages.
			MSG Msg = { };
			while (::PeekMessage(&Msg, NULL, 0, 0, PM_REMOVE))
			{
				::TranslateMessage(&Msg);
				::DispatchMessage(&Msg);
			}
		}
	}

	bool GetPowerStatus(TOptional<bool>& OutACPowerConnected, TOptional<uint32>& OutBatteryPct)
	{
		bool bAvailable = false;
		SYSTEM_POWER_STATUS Status;
		if (GetSystemPowerStatus(&Status))
		{
			switch (Status.ACLineStatus)
			{
			case 0: // AC Offline
				OutACPowerConnected.Emplace(false);
				bAvailable = true;
				break;
			case 1: // AC Online
				OutACPowerConnected.Emplace(true);
				bAvailable = true;
				break;
			default: // Unknown
				break;
			}

			if (Status.BatteryLifePercent != 255) // Unknown
			{
				OutBatteryPct.Emplace(Status.BatteryLifePercent);
				bAvailable = true;
			}
		}
		return bAvailable;
	}
#else
	void InitPlatformSpecific(){}
	void TickPlatformSpecific(){}
	bool GetPowerStatus(TOptional<bool>& OutACPowerConnected, TOptional<uint32>& OutBatteryPct) { return false; }
#endif

} // CrashReportClientUtils


/**
 * Augments the default summary senders to perform a short analytis to detect if this is the session ended abnormally. The sender searches
 * for very specific keys published by the engine. Remember that CRC merges its summary with the monitored process summary (the Editor), so
 * it gets to see what was recorded (or not) by the monitored process.
 */
class FCrashReportClientAnalyticsSessionSummarySender : public FAnalyticsSessionSummarySender
{
public:
	FCrashReportClientAnalyticsSessionSummarySender(IAnalyticsProviderET& Provider)
		: FAnalyticsSessionSummarySender(Provider)
	{
	}

	virtual bool SendSessionSummary(const FString& UserId, const FString& AppId, const FString& AppVersion, const FString& SessionId, const TArray<FAnalyticsEventAttribute>& Properties) override
	{
		// CRC should only send one session (its own), but do a reset in case this convention changes.
		bAbnormalShutdown = false;
		bUserLoggingOut = false;

		// Analyze the report to be sent and try to figure out if this is an abnormal shutdown. They keys are taken from the engine analytics session summary. To prevent dependencies
		// between CRC and Engine analytics, we duplicate the keys here.
		if (const FAnalyticsEventAttribute* ShutdownTypeCode = Properties.FindByPredicate([](const FAnalyticsEventAttribute& Candidate) { return Candidate.GetName() == FAnalyticsSessionSummaryManager::ShutdownTypeCodeProperty.Key; }))
		{
			bAbnormalShutdown = (ShutdownTypeCode->GetValue() == LexToString((int32)EAnalyticsSessionShutdownType::Abnormal));
			if (bAbnormalShutdown)
			{
				if (const FAnalyticsEventAttribute* UserLoggingOut = Properties.FindByPredicate([](const FAnalyticsEventAttribute& Candidate) { return Candidate.GetName() == FAnalyticsSessionSummaryManager::IsUserLoggingOutProperty.Key; }))
				{
					bUserLoggingOut = (UserLoggingOut->GetValue() == LexToString(true));
				}
			}
		}

		// Send the report unmodified.
		return FAnalyticsSessionSummarySender::SendSessionSummary(UserId, AppId, AppVersion, SessionId, Properties);
	}

	/** Returns whether the last session sent was abnormally terminated. */
	bool IsAbnormalShutdown() const
	{
		return bAbnormalShutdown && !bUserLoggingOut;
	}

private:
	bool bAbnormalShutdown = false;
	bool bUserLoggingOut = false;
};


FCrashReportAnalyticsSessionSummary::FCrashReportAnalyticsSessionSummary()
	: SessionStartTimeSecs(FPlatformTime::Seconds())
	, bMonitoredAppDeathRecorded(false)
	, bShutdown(false)
{
	if (IsEnabled())
	{
		CrashReportClientUtils::InitPlatformSpecific();

		// Reserve the memory for the log string.
		DiagnosticLog.Reset(CrashReportClientUtils::MaxDiagnosticLogLen);
		DiagnosticLog.Append(FString::Printf(TEXT("CRC/Init:%s"), *FDateTime::UtcNow().ToString()));
	}
}

FCrashReportAnalyticsSessionSummary& FCrashReportAnalyticsSessionSummary::FCrashReportAnalyticsSessionSummary::Get()
{
	static FCrashReportAnalyticsSessionSummary Instance;
	return Instance;
}

void FCrashReportAnalyticsSessionSummary::Initialize(const FString& ProcessGroupId, uint32 ForProcessId)
{
	if (IsEnabled() && !SessionSummaryManager && !ProcessGroupId.IsEmpty())
	{
		SessionSummaryManager = MakeUnique<FAnalyticsSessionSummaryManager>(TEXT("CrashReportClient"), ProcessGroupId, ForProcessId);
		if (SessionSummaryManager)
		{
			constexpr uint32 ReservedFileCapacity = CrashReportClientUtils::MaxDiagnosticLogLen + (4 * 1024);
			PropertyStore = SessionSummaryManager->MakeStore(ReservedFileCapacity);
			if (PropertyStore)
			{
				FCoreDelegates::GetApplicationWillTerminateDelegate().AddRaw(this, &FCrashReportAnalyticsSessionSummary::OnApplicationWillTerminate);
				FCoreDelegates::OnHandleSystemError.AddRaw(this, &FCrashReportAnalyticsSessionSummary::OnHandleSystemError);

				CrcAnalyticsProperties::EngineVersion.Set(PropertyStore.Get(), FEngineVersion::Current().ToString(EVersionComponent::Changelist));
				CrcAnalyticsProperties::SummaryVersionNumber.Set(PropertyStore.Get(), CrcAnalyticsProperties::CrcAnalyticsSummaryVersion);
				CrcAnalyticsProperties::StartupTimestamp.Set(PropertyStore.Get(), FDateTime::UtcNow());
				CrcAnalyticsProperties::Timestamp.Set(PropertyStore.Get(), FDateTime::UtcNow());
				CrcAnalyticsProperties::TickCount.Set(PropertyStore.Get(), 0);
				CrcAnalyticsProperties::SessionDurationSecs.Set(PropertyStore.Get(), FMath::FloorToInt(static_cast<float>(FPlatformTime::Seconds() - SessionStartTimeSecs)));
				CrcAnalyticsProperties::DiagnosticLogs.Set(PropertyStore.Get(), DiagnosticLog, CrashReportClientUtils::MaxDiagnosticLogLen);
				CrcAnalyticsProperties::IsReportingCrash.Set(PropertyStore.Get(), false);
				CrcAnalyticsProperties::IsCollectingCrash.Set(PropertyStore.Get(), false);
				CrcAnalyticsProperties::IsProcessingCrash.Set(PropertyStore.Get(), false);
				CrcAnalyticsProperties::UserIsLoggingOut.Set(PropertyStore.Get(), false);
				CrcAnalyticsProperties::IsCrashing.Set(PropertyStore.Get(), false);
				CrcAnalyticsProperties::WasShutdown.Set(PropertyStore.Get(), false);
				CrcAnalyticsProperties::QuitSignalRecv.Set(PropertyStore.Get(), false);
				CrcAnalyticsProperties::ReportCount.Set(PropertyStore.Get(), 0);
				CrcAnalyticsProperties::EnsureCount.Set(PropertyStore.Get(), 0);
				CrcAnalyticsProperties::AssertCount.Set(PropertyStore.Get(), 0);
				CrcAnalyticsProperties::StallCount.Set(PropertyStore.Get(), 0);

				UpdatePowerStatus();
				Flush();

				GLog->AddOutputDevice(this);

				// CRC main thread might be busy processing a crash, so use a background thread to record important events that could otherwise be missed.
				AnalyticsThread = MakeUnique<FThread>(TEXT("AnalyticsMonitorThread"), [this, ForProcessId]()
				{
					// Try to open the process.
					FProcHandle MonitoredProcessHandle = OpenProcessForMonitoring(ForProcessId);
					if (!MonitoredProcessHandle.IsValid())
					{
						LogEvent(TEXT("CRC/OpenProcessFailed"));
						return;
					}

					CrashReportClientUtils::InitPlatformSpecific();
					double NextFlushTimeSecs = FPlatformTime::Seconds();
					bool bFlushedLowBattery = false;

					LogEvent(TEXT("CRC/Monitoring")); // About to enter the loop.

					while (!bShutdown)
					{
						CrcAnalyticsProperties::TickCount.Update(PropertyStore.Get(), [](uint32& Actual) { ++Actual; return true; });
						CrashReportClientUtils::TickPlatformSpecific();

						// Monitor the power level.
						bool bShouldFlush = UpdatePowerStatus();

						if (!FPlatformProcess::IsProcRunning(MonitoredProcessHandle))
						{
							OnMonitoredAppDeath(MonitoredProcessHandle);
							break;
						}
						else if (FPlatformTime::Seconds() > NextFlushTimeSecs || bShouldFlush)
						{
							// Flush to timestamp the session.
							Flush();
							NextFlushTimeSecs += CrashReportClientUtils::PropertyStoreFlushPeriod.GetTotalSeconds();
						}

						// Throttle the thread.
						FPlatformProcess::Sleep(CrashReportClientUtils::TickPeriod.GetTotalSeconds());
					}

					FPlatformProcess::CloseProc(MonitoredProcessHandle);
				});
			}
		}
	}
}

bool FCrashReportAnalyticsSessionSummary::IsValid() const
{
	return IsEnabled() && PropertyStore.IsValid();
}

void FCrashReportAnalyticsSessionSummary::Shutdown(IAnalyticsProviderET* AnalyticsProvider, TFunction<void()> HandleAbnormalShutdownFn)
{
	if (!IsValid())
	{
		return;
	}

	if (AnalyticsThread)
	{
		bShutdown = true;
		AnalyticsThread->Join();
		AnalyticsThread.Reset();
	}

	CrcAnalyticsProperties::WasShutdown.Set(PropertyStore.Get(), true);
	LogEvent(FString::Printf(TEXT("CRC/Shutdown:%s:%.1fs"), *FDateTime::UtcNow().ToString(), FPlatformTime::Seconds() - SessionStartTimeSecs));

	// Unregister from the core.
	FCoreDelegates::GetApplicationWillTerminateDelegate().RemoveAll(this);
	FCoreDelegates::OnHandleSystemError.RemoveAll(this);
	GLog->RemoveOutputDevice(this);

	// Flush and close CRC analytics summary.
	Flush();
	PropertyStore.Reset();

	// If CRC is allowed to send summary report on behalf of the monitored application.
	if (AnalyticsProvider)
	{
		// Set a summary sender that will intercept the set of properties and perform an analysis to detect an abnormal shutdown.
		TSharedPtr<FCrashReportClientAnalyticsSessionSummarySender> SummarySender = MakeShared<FCrashReportClientAnalyticsSessionSummarySender>(*AnalyticsProvider);
		SessionSummaryManager->SetSender(SummarySender);

		// Merge the principal process summary with CRC process summary and sends the session summary if CRC is the last process to exit.
		SessionSummaryManager->Shutdown();

		// If the summary sender detected an abnormal shutdown by checking the summary properties, report it.
		if (SummarySender->IsAbnormalShutdown() && HandleAbnormalShutdownFn)
		{
			HandleAbnormalShutdownFn();
		}
	}
	else // Discard CRC summary.
	{
		SessionSummaryManager->Shutdown(/*bDiscard*/true);
	}

	SessionSummaryManager.Reset();
}

void FCrashReportAnalyticsSessionSummary::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, const double Time)
{
	Serialize(V, Verbosity, Category);
}

void FCrashReportAnalyticsSessionSummary::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category)
{
	// Log the errors, especially the failed 'check()' with the callstack/message.
	if (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Fatal)
	{
		// Log but don't forward to UE logging system. The log is already originate from the logging system.
		LogEvent(TEXT("CRC/Error"), /*bForwardToUELog*/false);
		LogEvent(V, /*bForwardToUELog*/false);
	}
}

bool FCrashReportAnalyticsSessionSummary::CanBeUsedOnAnyThread() const
{
	return true;
}

bool FCrashReportAnalyticsSessionSummary::CanBeUsedOnMultipleThreads() const
{
	return true;
}

bool FCrashReportAnalyticsSessionSummary::CanBeUsedOnPanicThread() const
{
	return true;
}

void FCrashReportAnalyticsSessionSummary::LogEvent(const FString& Event)
{
	LogEvent(*Event);
}

void FCrashReportAnalyticsSessionSummary::LogEvent(const TCHAR* Event, bool bForwardToUELog)
{
	if (IsValid())
	{
		FScopeLock ScopedLock(&LoggerLock);
		TGuardValue<bool> ReentrantGuard(bLoggerReentrantGuard, true);
		if (ReentrantGuard.GetOriginalValue()) // Read the old value.
		{
			return; // Prevent renentrant logging.
		}

		AppendLog(Event);
	}

	// Prevent error logs coming from the logging system to be duplicated.
	if (bForwardToUELog)
	{
		UE_LOGF(LogCrashReportClientDiagnostics, Log, "%ls", Event);
	}
}

void FCrashReportAnalyticsSessionSummary::AppendLog(const TCHAR* Event)
{
	// Add the separator if some text is already logged.
	if (DiagnosticLog.Len())
	{
		DiagnosticLog.Append(TEXT("|"));
	}

	// Rotate the log if it gets too long.
	int32 FreeLen = CrashReportClientUtils::MaxDiagnosticLogLen - DiagnosticLog.Len();
	int32 EventLen = FCString::Strlen(Event);
	if (EventLen > FreeLen)
	{
		if (EventLen > CrashReportClientUtils::MaxDiagnosticLogLen)
		{
			DiagnosticLog.Reset(CrashReportClientUtils::MaxDiagnosticLogLen);
			EventLen = CrashReportClientUtils::MaxDiagnosticLogLen;
		}
		else
		{
			DiagnosticLog.RemoveAt(0, EventLen - FreeLen, EAllowShrinking::No); // Free space, remove the chars from the oldest events (in front).
		}
	}

	// Append the log entry and dump the log to the file.
	DiagnosticLog.AppendChars(Event, EventLen);

	// Update the diagnostic field into the session summary store.
	CrcAnalyticsProperties::DiagnosticLogs.Set(PropertyStore.Get(), DiagnosticLog);

	// Flush the store.
	Flush();
}

void FCrashReportAnalyticsSessionSummary::OnMonitoredAppDeath(FProcHandle& Handle)
{
	// The first thread to exchange successfull is allowed to update. No need to update twice for the same monitored process.
	bool Expected = false;
	if (IsValid() && bMonitoredAppDeathRecorded.compare_exchange_strong(Expected, true))
	{
		CrcAnalyticsProperties::MonitoredAppDeathTimestamp.Set(PropertyStore.Get(), FDateTime::UtcNow());
		LogEvent(TEXT("App/Death"));

		int32 ExitCode;
		if (Handle.IsValid() && FPlatformProcess::GetProcReturnCode(Handle, &ExitCode))
		{
			CrcAnalyticsProperties::MonitoredAppExitCode.Set(PropertyStore.Get(), ExitCode);
			LogEvent(FString::Printf(TEXT("App/ExitCode:%d"), ExitCode));
		}
		else
		{
			CrcAnalyticsProperties::MonitoredAppExitCode.Set(PropertyStore.Get(), ECrashExitCodes::MonitoredApplicationExitCodeNotAvailable);
			LogEvent(TEXT("App/ExitCode:N/A"));
		}
		Flush();
	}
}

void FCrashReportAnalyticsSessionSummary::OnUserLoggingOut()
{
	if (IsValid())
	{
		// The user is logging out and CRC is going to die.
		CrcAnalyticsProperties::UserIsLoggingOut.Set(PropertyStore.Get(), true);

		// Log the event (this also flush the session).
		LogEvent(TEXT("CRC/EndSession"));
	}
}

void FCrashReportAnalyticsSessionSummary::OnQuitSignal()
{
	// The system has requested the app to close. (Like if the user gently kills the application)
	CrcAnalyticsProperties::QuitSignalRecv.Set(PropertyStore.Get(), true);

	// Log the event (this also flush the session).
	LogEvent(TEXT("CRC/QuitSignal"));
}

void FCrashReportAnalyticsSessionSummary::OnCrcCrashing(int32 ExceptCode)
{
	if (IsValid())
	{
		CrcAnalyticsProperties::IsCrashing.Set(PropertyStore.Get(), true);
		CrcAnalyticsProperties::ExceptCode.Set(PropertyStore.Get(), ExceptCode);
	
		TCHAR CrashEventLog[64];
		FCString::Sprintf(CrashEventLog, TEXT("CRC/Crash:%d"), ExceptCode);
		LogEvent(CrashEventLog); // This also flush the session.
	}
}

void FCrashReportAnalyticsSessionSummary::Flush()
{
	if (IsValid())
	{
		// Update the session progression.
		CrcAnalyticsProperties::Timestamp.Set(PropertyStore.Get(), FDateTime::UtcNow(),
			[](const FDateTime* Actual, const FDateTime& Proposed) { return Proposed > *Actual; });

		CrcAnalyticsProperties::SessionDurationSecs.Set(PropertyStore.Get(), FMath::FloorToInt(FPlatformTime::Seconds() - SessionStartTimeSecs),
			[](const int32* Actual, const int32& Proposed) { return Proposed > *Actual; });

		// Flush the store to disk.
		PropertyStore->Flush();
	}
}

void FCrashReportAnalyticsSessionSummary::OnApplicationWillTerminate()
{
	LogEvent(FString::Printf(TEXT("CRC/Terminate:%s"), *FDateTime::UtcNow().ToString()));
}

void FCrashReportAnalyticsSessionSummary::OnHandleSystemError()
{
	CrcAnalyticsProperties::IsCrashing.Set(PropertyStore.Get(), true);
	LogEvent(FString::Printf(TEXT("CRC/SysError:%s"), *FDateTime::UtcNow().ToString()));
}

void FCrashReportAnalyticsSessionSummary::OnCrashReportStarted(ECrashContextType CrashType, const TCHAR* ErrorMsg)
{
	CrashReportStartTimeSecs = FPlatformTime::Seconds();

	// Reset the timers for the current crash report (a safety measure).
	CrashReportCollectingStartTimeSecs = CrashReportStartTimeSecs;
	CrashReportStackWalkingStartTimeSecs = CrashReportStartTimeSecs;
	CrashReportGatheringFilesStartTimeSecs = CrashReportStartTimeSecs;
	CrashReportSignalingRemoteAppTimeSecs = CrashReportStartTimeSecs;
	CrashReportProcessingStartTimeSecs = CrashReportStartTimeSecs;

	CrcAnalyticsProperties::IsReportingCrash.Set(PropertyStore.Get(), true);
	CrcAnalyticsProperties::ReportCount.Update(PropertyStore.Get(), [](uint32& Actual) { ++Actual; return true; });
	LogEvent(FString::Printf(TEXT("Report/Start:%s"), *FDateTime::UtcNow().ToString()));

	// Log the assert and ensure condition/file/line/message to the diagnostic log gathered by the analytics to enable searching/grouping them later on.
	if (CrashType == ECrashContextType::Assert)
	{
		CrcAnalyticsProperties::AssertCount.Update(PropertyStore.Get(), [](uint32& Actual) { ++Actual; return true; });
		LogEvent(FString::Printf(TEXT("Assert/Msg: %s"), ErrorMsg));
	}
	else if (CrashType == ECrashContextType::Ensure)
	{
		CrcAnalyticsProperties::EnsureCount.Update(PropertyStore.Get(), [](uint32& Actual) { ++Actual; return true; });

		// Ensure messages include the ensure call stack. That's not useful for analytics, try keeping the essential only (ensure condition, file, line)
		FRegexPattern Pattern(TEXT(R"(.*\[File:.*\]\s*\[Line:\s\d+\])")); // Need help with regex? Try https://regex101.com/
		FRegexMatcher Matcher(Pattern, ErrorMsg);
		if (Matcher.FindNext())
		{
			LogEvent(FString::Printf(TEXT("Ensure/Msg: %s"), *Matcher.GetCaptureGroup(0)));
		}
		else
		{
			LogEvent(FString::Printf(TEXT("Ensure/Msg: %s"), ErrorMsg));
		}
	}
	else if (CrashType == ECrashContextType::Stall)
	{
		CrcAnalyticsProperties::StallCount.Update(PropertyStore.Get(), [](uint32& Actual) { ++Actual; return true; });
	}
}

void FCrashReportAnalyticsSessionSummary::OnCrashReportCollecting()
{
	CrashReportCollectingStartTimeSecs = FPlatformTime::Seconds();

	CrcAnalyticsProperties::IsCollectingCrash.Set(PropertyStore.Get(), true);
	FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Report/Collect"));
}

void FCrashReportAnalyticsSessionSummary::OnCrashReportRemoteStackWalking()
{
	CrashReportStackWalkingStartTimeSecs = FPlatformTime::Seconds();
}

void FCrashReportAnalyticsSessionSummary::OnCrashReportGatheringFiles()
{
	CrashReportGatheringFilesStartTimeSecs = FPlatformTime::Seconds();
}

void FCrashReportAnalyticsSessionSummary::OnCrashReportSignalingAppToResume()
{
	CrashReportSignalingRemoteAppTimeSecs = FPlatformTime::Seconds();
}

void FCrashReportAnalyticsSessionSummary::OnCrashReportProcessing(bool bUserInteractive)
{
	CrcAnalyticsProperties::IsCollectingCrash.Set(PropertyStore.Get(), false);
	CrcAnalyticsProperties::IsProcessingCrash.Set(PropertyStore.Get(), true);
	CrashReportProcessingStartTimeSecs = FPlatformTime::Seconds();
	bProcessingCrashUnattended = !bUserInteractive;
	FCrashReportAnalyticsSessionSummary::Get().LogEvent(FString::Printf(TEXT("Report/Process:%s"), bUserInteractive ? TEXT("Interactive") : TEXT("Unattended")));
}

void FCrashReportAnalyticsSessionSummary::OnCrashReportCompleted(bool bSubmitted)
{
	CrcAnalyticsProperties::IsCollectingCrash.Set(PropertyStore.Get(), false);
	CrcAnalyticsProperties::IsProcessingCrash.Set(PropertyStore.Get(), false);
	CrcAnalyticsProperties::IsReportingCrash.Set(PropertyStore.Get(), false);

	double CurrTimeSecs = FPlatformTime::Seconds();

	// Total time required to remote stack walk, gather files, respond to the monited app.
	double CollectTimeSecs = CrashReportProcessingStartTimeSecs - CrashReportCollectingStartTimeSecs;

	// Total time required to remote stack walk
	double StackWalkSecs = CrashReportGatheringFilesStartTimeSecs - CrashReportStackWalkingStartTimeSecs;

	// Total time required to gather files (copy log and generates minidump).
	double GatherFileSecs = CrashReportSignalingRemoteAppTimeSecs - CrashReportGatheringFilesStartTimeSecs;

	// Total time required by CRC to process the crash report (resolve symbols + showing the UI + user time to respond if the report is interactive).
	double ProcessTimeSecs = CurrTimeSecs - CrashReportProcessingStartTimeSecs;

	// Total time CRC main thread was used to process the crash.
	double TotalTimeSecs = CurrTimeSecs - CrashReportStartTimeSecs;

	if (bProcessingCrashUnattended) // No UI shown to the user that could amplify the time.
	{
		CrcAnalyticsProperties::LonguestUnattendedReportSecs.Set(PropertyStore.Get(), static_cast<float>(TotalTimeSecs), [](const float* Actual, const float& Proposed) { return Actual == nullptr || Proposed > *Actual; });
	}
	bProcessingCrashUnattended = false;

	const TCHAR* Event = bSubmitted ? TEXT("Report/Sent") : TEXT("Report/Discarded");
	LogEvent(FString::Printf(TEXT("%s:Walk=%.1fs:Gather=%.1fs:Collect=%.1fs:Process=%.1fs:Total=%.1fs"), Event, StackWalkSecs, GatherFileSecs, CollectTimeSecs, ProcessTimeSecs, TotalTimeSecs));
}

bool FCrashReportAnalyticsSessionSummary::UpdatePowerStatus()
{
	bool bShouldFlush = false;

	// Monitor the power level.
	TOptional<bool> ConnectedToACPower;
	TOptional<uint32> BatteryPercentage;
	if (CrashReportClientUtils::GetPowerStatus(ConnectedToACPower, BatteryPercentage))
	{
		if (ConnectedToACPower)
		{
			CrcAnalyticsProperties::IsOnACPower.Set(PropertyStore.Get(), *ConnectedToACPower);
		}
		if (BatteryPercentage)
		{
			CrcAnalyticsProperties::BatteryLevel.Set(PropertyStore.Get(), *BatteryPercentage, [&bShouldFlush](const uint32* PreviousLevel, const uint32& NewLevel)
			{
				// Detects when the batter level goes from above to below or equal a threshonld.
				constexpr uint32 LowBatteryPct = 2;
				if (PreviousLevel && *PreviousLevel > LowBatteryPct && NewLevel <= LowBatteryPct)
				{
					// Last time, it was above the threshold, but now it dipped below, save the state before the battery runs out.
					bShouldFlush = true;
				}
				return true;
			});
		}
	}

	return bShouldFlush;
}
