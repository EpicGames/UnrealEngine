// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/PlatformProcess.h"
#include "Misc/OutputDevice.h"
#include "Templates/Function.h"
#include "CrashReportClientDefines.h"
#include <atomic>

class IAnalyticsPropertyStore;
class IAnalyticsProviderET;
class FAnalyticsSessionSummaryManager;
class FThread;

/** Open the monitored process with restricted permissions, for security reasons. */
FProcHandle OpenProcessForMonitoring(uint32 pid);

/**
 * Creates a summary of CRC session for analytics purpose. The summary should be merged with the
 * monitored application one.
 */
class FCrashReportAnalyticsSessionSummary : public FOutputDevice
{
public:
	/** Return the summary instance. */
	static FCrashReportAnalyticsSessionSummary& Get();

	/** Initialize the summary. The implementation do nothing unless the instance is successfully initialized. */
	void Initialize(const FString& ProcessGroupId, uint32 ForProcessId);

	/** Shuts down and submits the summary if a non-null provider is specified and invoke HandleAbnormalShutdownFn if bound and an abnormal shutdown was detected. */
	void Shutdown(IAnalyticsProviderET* AnalyticsProvider = nullptr, TFunction<void()> HandleAbnormalShutdownFn = TFunction<void()>());

	//~ FOutputDevice interface
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, const double Time) override;
	virtual bool CanBeUsedOnAnyThread() const override;
	virtual bool CanBeUsedOnMultipleThreads() const override;
	virtual bool CanBeUsedOnPanicThread() const override;

	/** Logs an events in the analytics session summary diagnostic log. The event text is expected to be short and concise. */
	void LogEvent(const TCHAR* Event, bool bForwardToUELog = true);

	/** Logs an events in the analytics session summary diagnostic log. The event text is expected to be short and concise. */
	void LogEvent(const FString& Event);

	/** Invoked when monitored application death was detected. */
	void OnMonitoredAppDeath(FProcHandle& Handle);

	/** Invoked by the unhandled exception handler while CRC is crashing. */
	void OnCrcCrashing(int32 ExceptCode);

	/** Invoked when the user is logging out. This is also called if the computer is shutting down or restarting. */
	void OnUserLoggingOut();

	/** Invoked when the system gently signal CRC to quit. */
	void OnQuitSignal();

	/** Invoked when CRC starts to process a crash/ensure/stall.*/
	void OnCrashReportStarted(ECrashContextType Type, const TCHAR* ErrorMsg);

	/** Invoked when CRC starts collecting the crash artifacts. */
	void OnCrashReportCollecting();

	/** Invoked when CRC starts stack-walking the crash. */
	void OnCrashReportRemoteStackWalking();

	/** Invoked when CRC starts collecting crash artifacts such as the log/minidump/etc. */
	void OnCrashReportGatheringFiles();

	/** Invoked when CRC finished collecting crash artifacts and replies to the monitored app. */
	void OnCrashReportSignalingAppToResume();

	/** Invoked when CRC starts the process to submit the crash report (this also includes showing the dialog to user in needed. */
	void OnCrashReportProcessing(bool bUserInteractive);

	/** Invoked when CRC finished handling a crash report and resume its main loop. */
	void OnCrashReportCompleted(bool bSubmitted);

private:
	/** Returns whether the analytics summary is enabled or not. */
	static bool IsEnabled() { return CRASH_REPORT_WITH_MTBF != 0; }

	/** Returns true if the analytics session summary is valid. */
	bool IsValid() const;

	/** Creates the summary sessions. */
	FCrashReportAnalyticsSessionSummary();

	/** Appends a log entry to the log buffer, rotate the buffer if full and flush it to file. */
	void AppendLog(const TCHAR* Event);

	/** Flush CRC analytics summary file to disk. */
	void Flush();

	/** Invoked by the core when the application is about to terminate. */
	void OnApplicationWillTerminate();

	/** Invoked by the core when a system error is detected, like fatal logs. */
	void OnHandleSystemError();

	/** Invoked periodically by the monitoring thread to update the power status (Battery/AC Power). */
	bool UpdatePowerStatus();

private:
	/** Serialize access to the diagnostic logs. */
	FCriticalSection LoggerLock;

	/** The CRC session summary store. */
	TSharedPtr<IAnalyticsPropertyStore> PropertyStore;

	/** The session summary manager. */
	TUniquePtr<FAnalyticsSessionSummaryManager> SessionSummaryManager;

	/** Contains the rotating diagnostic log entries. */
	FString DiagnosticLog;

	/** Runs in background to collect CRC analytics because the main thread might be busy handling a crash and important event could be missed. */
	TUniquePtr<FThread> AnalyticsThread;

	/** The CRC startup time, i.e. when the summary is successfully initialized. */
	double SessionStartTimeSecs;

	/** Time at which the last crash report started. */
	double CrashReportStartTimeSecs = 0.0;

	/** Time at which collecting the artifacts of the last crash started. */
	double CrashReportCollectingStartTimeSecs = 0.0;

	/** Time at which remote stack-walking started. */
	double CrashReportStackWalkingStartTimeSecs = 0.0;

	/** Time at which gathering crash files (logs, minidump, etct) started. */
	double CrashReportGatheringFilesStartTimeSecs = 0.0;

	/** Time at which CRC signaled the application to resume. */
	double CrashReportSignalingRemoteAppTimeSecs = 0.0;

	/** Time at which processing the last crash started. */
	double CrashReportProcessingStartTimeSecs = 0.0;

	/** Whether the crash being processed unattended. */
	bool bProcessingCrashUnattended = false;

	/** Prevent a reentrency in the diagnostic logger. */
	bool bLoggerReentrantGuard = false;

	/** Flag if the session is usable. */
	bool bIsValid = false;

	/** Indicates whether the monitored application death is reported. */
	std::atomic<bool> bMonitoredAppDeathRecorded;

	/** Used to signal the analytics thread to exit. */
	std::atomic<bool> bShutdown;
};
