// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportClient.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Internationalization.h"
#include "Containers/Ticker.h"
#include "CrashReportCoreConfig.h"
#include "Templates/UniquePtr.h"
#include "Async/TaskGraphInterfaces.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "CrashReportClient"

struct FCrashReportUtil
{
	/** Formats processed diagnostic text by adding additional information about machine and user. */
	static FText FormatDiagnosticText( const FText& DiagnosticText )
	{
		TStringBuilder<512> Accounts;
		if (const FString LoginId = FPrimaryCrashProperties::Get()->LoginId.AsString(); !LoginId.IsEmpty())
		{
			Accounts.Appendf(TEXT("LoginId:%s\n"), *LoginId);
		}
		if (const FString EpicAccountId= FPrimaryCrashProperties::Get()->EpicAccountId.AsString(); !EpicAccountId.IsEmpty())
		{
			Accounts.Appendf(TEXT("EpicAccountId:%s\n"), *EpicAccountId);
		}
		return FText::Format(LOCTEXT("CrashReportClientCallstackPattern", "{0}\n{1}"), FText::FromString(Accounts.ToString()), DiagnosticText);
	}
};

#if !CRASH_REPORT_UNATTENDED_ONLY

#include "PlatformHttp.h"
#include "Framework/Application/SlateApplication.h"

FCrashReportClient::FCrashReportClient(const FPlatformErrorReport& InErrorReport, bool bImplicitSend)
	: DiagnosticText( LOCTEXT("ProcessingReport", "Processing crash report ...") )
	, DiagnoseReportTask(nullptr)
	, ErrorReport( InErrorReport )
	, ReceiverUploader(FCrashReportCoreConfig::Get().GetReceiverAddress())
	, DataRouterUploader(FCrashReportCoreConfig::Get().GetDataRouterURL(), FCrashReportCoreConfig::Get().GetDataRouterUnixSocket())
	, bShouldWindowBeHidden(false)
	, bSendData(bImplicitSend)
	, bSendOptionalAttachments(false)
	, bIsSuccesfullRestart(false)
	, bIsUploadComplete(false)
{
	if (FPrimaryCrashProperties::Get()->IsValid())
	{
		bool bUsePrimaryData = false;
		if (FPrimaryCrashProperties::Get()->HasProcessedData())
		{
			bUsePrimaryData = true;
		}
		else
		{
			if (!ErrorReport.TryReadDiagnosticsFile() && !FParse::Param( FCommandLine::Get(), TEXT( "no-local-diagnosis" ) ))
			{
				DiagnoseReportTask = new FAsyncTask<FDiagnoseReportWorker>( this );
				DiagnoseReportTask->StartBackgroundTask();
				StartTicker();
			}
			else
			{
				bUsePrimaryData = true;
			}
		}

		if (bUsePrimaryData)
		{
			const FString CallstackString = FPrimaryCrashProperties::Get()->CallStack.AsString();
			const FString ReportString = FString::Printf( TEXT( "%s\n\n%s" ), *FPrimaryCrashProperties::Get()->ErrorMessage.AsString(), *CallstackString );
			DiagnosticText = FText::FromString( ReportString );

			FormattedDiagnosticText = FCrashReportUtil::FormatDiagnosticText( FText::FromString( ReportString ) );
		}
	}

	if (bSendData)
	{
		StartTicker();
	}
}

FCrashReportClient::~FCrashReportClient()
{
	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
	StopBackgroundThread();
}

void FCrashReportClient::StopBackgroundThread()
{
	if (DiagnoseReportTask)
	{
		DiagnoseReportTask->EnsureCompletion();
		delete DiagnoseReportTask;
		DiagnoseReportTask = nullptr;
	}
}

FReply FCrashReportClient::CloseWithoutSending()
{
	bSendData = false;
	bShouldWindowBeHidden = true;
	StartTicker();
	return FReply::Handled();
}

FReply FCrashReportClient::Close()
{
	bShouldWindowBeHidden = true;
	return FReply::Handled();
}

#if PLATFORM_WINDOWS
extern void CopyDiagnosticFilesToClipboard(TConstArrayView<FString> Files);
#endif

#if PLATFORM_WINDOWS
FReply FCrashReportClient::CopyFilesToClipboard()
{
	TArray<FString> Files = FPlatformErrorReport(ErrorReport.GetReportDirectory()).GetFilesToUpload();
	CopyDiagnosticFilesToClipboard(Files);
	return FReply::Handled();
}
#endif

FReply FCrashReportClient::Submit(bool bIncludeOptionalAttachments)
{
	bSendData = true;
	StoreCommentAndUpload();

	if (bIncludeOptionalAttachments)
	{
		bSendOptionalAttachments = true;
	}

	bShouldWindowBeHidden = true;
	return FReply::Handled();
}

FReply FCrashReportClient::SubmitOptionalAttachmentsAndClose()
{
	bSendOptionalAttachments = true;
	bShouldWindowBeHidden = true;
	return FReply::Handled();
}

FReply FCrashReportClient::SubmitAndRestart()
{
	Submit();

	// Check for processes that were started from the Launcher using -EpicPortal on the command line
	bool bRunFromLauncher = FParse::Param(*FPrimaryCrashProperties::Get()->RestartCommandLine, TEXT("EPICPORTAL"));
	const FString CrashedAppPath = ErrorReport.FindCrashedAppPath();

	bool bLauncherRestarted = false;
	if (bRunFromLauncher)
	{
		// Hacky check to see if this is the editor. Not attempting to relaunch the editor using the Launcher because there is no way to pass the project via OpenLauncher()
		if (!FPaths::GetCleanFilename(CrashedAppPath).StartsWith(TEXT("UnrealEditor")))
		{
			// We'll restart Launcher-run processes by having the installed Launcher handle it
			ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();

			if (LauncherPlatform != nullptr)
			{
				// Split the path so we can format it as a URI
				TArray<FString> PathArray;
				CrashedAppPath.Replace(TEXT("//"), TEXT("/")).ParseIntoArray(PathArray, TEXT("/"), false);	// WER saves this out on Windows with double slashes as the separator for some reason.
				FString CrashedAppPathUri;

				// Exclude the last item (the filename). The Launcher currently expects an installed application folder.
				for (int32 ItemIndex = 0; ItemIndex < PathArray.Num() - 1; ItemIndex++)
				{
					FString& PathItem = PathArray[ItemIndex];
					CrashedAppPathUri += FPlatformHttp::UrlEncode(PathItem);
					CrashedAppPathUri += TEXT("/");
				}
				CrashedAppPathUri.RemoveAt(CrashedAppPathUri.Len() - 1);

				// Re-run the application via the Launcher
				FOpenLauncherOptions OpenOptions(FString::Printf(TEXT("apps/%s?action=launch"), *CrashedAppPathUri));
				OpenOptions.bSilent = true;
				if (LauncherPlatform->OpenLauncher(OpenOptions))
				{
					bLauncherRestarted = true;
					bIsSuccesfullRestart = true;
				}
			}
		}
	}

	if (!bLauncherRestarted)
	{
		// Launcher didn't restart the process so start it ourselves
		const FString CommandLineArguments = FPrimaryCrashProperties::Get()->RestartCommandLine;
		FPlatformProcess::CreateProc(*CrashedAppPath, *CommandLineArguments, true, false, false, NULL, 0, NULL, NULL);
		bIsSuccesfullRestart = true;
	}

	return FReply::Handled();
}

FReply FCrashReportClient::CopyCallstack()
{
	FPlatformApplicationMisc::ClipboardCopy(*DiagnosticText.ToString());
	return FReply::Handled();
}

FText FCrashReportClient::GetDiagnosticText() const
{
	return FormattedDiagnosticText;
}

void FCrashReportClient::UserCommentChanged(const FText& Comment, ETextCommit::Type CommitType)
{
	UserComment = Comment;

	// Implement Shift+Enter to commit shortcut
	if (CommitType == ETextCommit::OnEnter && FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		Submit();
	}
}

void FCrashReportClient::RequestCloseWindow(const TSharedRef<SWindow>& Window)
{
	// We may still processing minidump etc. so start the main ticker.
	StartTicker();
	bShouldWindowBeHidden = true;
}

bool FCrashReportClient::AreCallstackWidgetsEnabled() const
{
	return !IsProcessingCallstack();
}

EVisibility FCrashReportClient::IsThrobberVisible() const
{
	return IsProcessingCallstack() ? EVisibility::Visible : EVisibility::Hidden;
}

void FCrashReportClient::AllowToBeContacted_OnCheckStateChanged( ECheckBoxState NewRadioState )
{
	FCrashReportCoreConfig::Get().SetAllowToBeContacted( NewRadioState == ECheckBoxState::Checked );

	// Refresh PII based on the bAllowToBeContacted flag.
	FPrimaryCrashProperties::Get()->UpdateIDs();

	// Save updated properties.
	FPrimaryCrashProperties::Get()->Save();

	// Update diagnostics text.
	FormattedDiagnosticText = FCrashReportUtil::FormatDiagnosticText( DiagnosticText );
}

void FCrashReportClient::SendLogFile_OnCheckStateChanged( ECheckBoxState NewRadioState )
{
	FCrashReportCoreConfig::Get().SetSendLogFile( NewRadioState == ECheckBoxState::Checked );
}

void FCrashReportClient::StartTicker()
{
	if (!TickHandle.IsValid())
	{
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FCrashReportClient::Tick), 1.f);
	}
}

void FCrashReportClient::StoreCommentAndUpload()
{
	// Write user's comment
	ErrorReport.SetUserComment( UserComment );
	StartTicker();
}

bool FCrashReportClient::Tick(float UnusedDeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FCrashReportClient_Tick);

	// We are waiting for diagnose report task to complete.
	if( IsProcessingCallstack() )
	{
		return true;
	}

	if (DiagnoseReportTask)
	{
		check(DiagnoseReportTask->IsWorkDone()); // Expected when IsProcessingCallstack() returns false.
		StopBackgroundThread(); // Free the DiagnoseReportTask to avoid reentering this condition.
		FinalizeDiagnoseReportWorker(); // Update the Text displaying call stack information (on game thread as they are visualized in UI)
		check(DiagnoseReportTask == nullptr); // Expected after StopBackgroundThread() call.
	}

	// Implicit send will begin uploading immediately and continue after the window is hidden
	if( bSendData )
	{
		if (!FCrashUploadBase::IsInitialized())
		{
			FCrashUploadBase::StaticInitialize( ErrorReport );
		}

		if (ReceiverUploader.IsEnabled())
		{
			if (!ReceiverUploader.IsUploadCalled())
			{
				// Can be called only when we have all files.
				ReceiverUploader.BeginUpload(ErrorReport);
			}

			// IsWorkDone will always return true here (since ReceiverUploader can't finish until the diagnosis has been sent), but it
			//  has the side effect of joining the worker thread.
			if (!ReceiverUploader.IsFinished())
			{
				// More ticks, please
				return true;
			}
		}

		if (DataRouterUploader.IsEnabled())
		{
			if (!DataRouterUploader.IsUploadCalled())
			{
				// Can be called only when we have all files.
				DataRouterUploader.BeginUpload(ErrorReport);
			}

			// IsWorkDone will always return true here (since DataRouterUploader can't finish until the diagnosis has been sent), but it
			//  has the side effect of joining the worker thread.
			if (!DataRouterUploader.IsFinished())
			{
				// More ticks, please
				return true;
			}
		}
	}

	// Optional attachments are only supported by the Data Router uploader.
	if (bSendData && bSendOptionalAttachments && DataRouterUploader.IsEnabled())
	{
		UE_LOGF(CrashReportClientLog, Log, "User has requested to upload optional attachments");

		// Wait until the base report has been sent (successfully or not).
		if (!DataRouterUploader.IsFinished())
		{
			return true;
		}

		// Only send optional attachments if the base report was uploaded successfully.
		if (DataRouterUploader.IsFinishedSuccessfully())
		{
			// Start upload and keep ticking until done.
			DataRouterUploader.BeginUploadOptionalAttachments(ErrorReport);

			bSendOptionalAttachments = false;
			return true;
		}
		else
		{
			UE_LOGF(CrashReportClientLog, Warning, "User has requested to upload optional attachments but the base report wasn't uploaded successfully: cancelling optional attachments upload");
			bSendOptionalAttachments = false;
		}
	}

	if (!bShouldWindowBeHidden)
	{
		return true;
	}

	if (FCrashUploadBase::IsInitialized())
	{
		FCrashUploadBase::StaticShutdown();
	}

	bIsUploadComplete = true;
	return false;
}

FString FCrashReportClient::GetCrashDirectory() const
{
	return ErrorReport.GetReportDirectory();
}

void FCrashReportClient::FinalizeDiagnoseReportWorker()
{
	// Update properties for the crash.
	ErrorReport.SetPrimaryCrashProperties( *FPrimaryCrashProperties::Get() );

	FString CallstackString = FPrimaryCrashProperties::Get()->CallStack.AsString();
	if (CallstackString.IsEmpty())
	{
		if (FPrimaryCrashProperties::Get()->PCallStackHash.IsEmpty())
		{
			DiagnosticText = LOCTEXT( "NoCallstack", "The system failed to capture the callstack for this crash." );
		}
		else
		{
			DiagnosticText = LOCTEXT( "NoDebuggingSymbols", "You do not have any debugging symbols required to display the callstack for this crash." );
		}
	}
	else
	{
		const FString ReportString = FString::Printf( TEXT( "%s\n\n%s" ), *FPrimaryCrashProperties::Get()->ErrorMessage.AsString(), *CallstackString );
		DiagnosticText = FText::FromString( ReportString );
	}

	FormattedDiagnosticText = FCrashReportUtil::FormatDiagnosticText( DiagnosticText );
}

bool FCrashReportClient::IsProcessingCallstack() const
{
	return DiagnoseReportTask && !DiagnoseReportTask->IsWorkDone();
}

FDiagnoseReportWorker::FDiagnoseReportWorker( FCrashReportClient* InCrashReportClient ) 
	: CrashReportClient( InCrashReportClient )
{}

void FDiagnoseReportWorker::DoWork()
{
	CrashReportClient->ErrorReport.DiagnoseReport();
}

#endif // !CRASH_REPORT_UNATTENDED_ONLY

#undef LOCTEXT_NAMESPACE
