// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "CrashReportClientApp.h"
#include "CrashUpload.h"
#include "Containers/Ticker.h"

DECLARE_LOG_CATEGORY_EXTERN(CrashReportClientLog, Log, All);

#if !CRASH_REPORT_UNATTENDED_ONLY

#include "Input/Reply.h"
#include "Layout/Visibility.h"

class SWindow;
enum class ECheckBoxState : uint8;
class FCrashReportClient;

/**
 * Helper task class to process a crash report in the background
 */
class FDiagnoseReportWorker  : public FNonAbandonableTask
{
public:
	/** Pointer to the crash report client, used to store the results. */
	FCrashReportClient* CrashReportClient;

	/** Initialization constructor. */
	FDiagnoseReportWorker( FCrashReportClient* InCrashReportClient );

	/**
	 * Do platform-specific work to get information about the crash.
	 */
	void DoWork();

	FORCEINLINE TStatId GetStatId() const
	{
		return TStatId();
	}

	/** 
	 * @return The name to display in external event viewers
	 */
	static const TCHAR* Name()
	{
		return TEXT( "FDiagnoseCrashWorker" );
	}
};

/**
 * Main implementation of the crash report client application
 */
class FCrashReportClient : public TSharedFromThis<FCrashReportClient>
{
	friend class FDiagnoseReportWorker;

public:
	/**
	 * Constructor: sets up background diagnosis
	 * @param ErrorReport Error report to upload
	 */
	FCrashReportClient( const FPlatformErrorReport& InErrorReport, bool bImplicitSend );

	/** Destructor. */
	virtual ~FCrashReportClient();

	/** Stops processing work in the background. */
	void StopBackgroundThread();

	/** Closes the crash report client without sending any data. Except the startup analytics. */
	FReply CloseWithoutSending();

	/** Closes the crash report client allowing data to finish being sent. */
	FReply Close();
	
#if PLATFORM_WINDOWS
	/** Copy report files to the clipboard for user to immediately request help with. */
	FReply CopyFilesToClipboard();
#endif

	/**
	 * Respond to the user pressing Submit
	 * @return Whether the request was handled
	 */
	FReply Submit(bool bIncludeOptionalAttachments = false);

	/**
	 * Respond to the user pressing Submit for optional attachments
	 * @return Whether the request was handled
	 */
	FReply SubmitOptionalAttachmentsAndClose();

	/**
	* Respond to the user pressing Submit and Restart
	* @return Whether the request was handled
	*/
	FReply SubmitAndRestart();

	/**
	 * Respond to the user requesting the callstack to be copied to the clipboard
	 * @return Whether the request was handled
	 */
	FReply CopyCallstack();

	/**
	 * Pass on exception and callstack from the platform error report code
	 * @return Localized text to display
	 */
	FText GetDiagnosticText() const;

	/**
	 * @return the full path of the crash directory.
	 */
	FString GetCrashDirectory() const;

	/**
	 * Handle the user updating the user comment text
	 * @param Comment Text provided by the user
	 * @param CommitType Event that caused this update
	 */
	void UserCommentChanged(const FText& Comment, ETextCommit::Type CommitType);

	/**
	 * Handle user closing the main window
	 * @param Window Main window
	 */
	void RequestCloseWindow(const TSharedRef<SWindow>& Window);

	/** Whether the main window should be hidden. */
	bool ShouldWindowBeHidden() const
	{
		return bShouldWindowBeHidden;
	}

	/** Whether the app should enable widgets related to the displayed callstack. */
	bool AreCallstackWidgetsEnabled() const;

	/** Whether the throbber should be visible while processing the callstack. */
	EVisibility IsThrobberVisible() const;

	/** Returns true if user has elected to restart crashing process, and if it was a success */
	bool GetIsSuccesfullRestart() const { return bIsSuccesfullRestart; }

	/** Retruns true if user has elected to close without sending the report. */
	bool WasClosedWithoutSending() const { return !bSendData; }

	void AllowToBeContacted_OnCheckStateChanged( ECheckBoxState NewRadioState );
	void SendLogFile_OnCheckStateChanged( ECheckBoxState NewRadioState );

	bool IsUploadComplete() const { return bIsUploadComplete; }

private:
	/**
	 * Write the user's comment to the report and begin uploading the entire report 
	 */
	void StoreCommentAndUpload();

	/**
	 * Update received every second
	 * @param DeltaTime Time since last update, unused
	 * @return Whether the updates should continue
	 */
	bool Tick(float DeltaTime);

	/**
	 * Begin calling Tick once a second
	 */
	void StartTicker();

	/** Enqueued from the diagnose report worker thread to be executed on the game thread. */
	void FinalizeDiagnoseReportWorker();

	/**
	 * @return true if we are still processing a callstack
	 */
	bool IsProcessingCallstack() const;

	/** Comment provided by the user */
	FText UserComment;

	/** Exception and call-stack to show, valid once diagnosis task is complete */
	FText DiagnosticText;

	/** Formatted diagnostics crash reporter data. */
	FText FormattedDiagnosticText;

	/** Background worker to get a callstack from the report */
	FAsyncTask<FDiagnoseReportWorker>* DiagnoseReportTask;

	/** Platform code for accessing the report */
	FPlatformErrorReport ErrorReport;

	/** Object that uploads report files to the server */
	FCrashUploadToReceiver ReceiverUploader;

	/** Object that uploads report files to the server */
	FCrashUploadToDataRouter DataRouterUploader;

	/** Whether the main window should be hidden. */
	bool bShouldWindowBeHidden;

	/** Whether we send the data. */
	bool bSendData;

	/** Whether to send optional attachments, after the base crash report has been successfully sent. */
	bool bSendOptionalAttachments;

	/** Store if user has elected to restart crashing process, and if it was a success.*/
	bool bIsSuccesfullRestart;

	/** Is the uploading complete. */
	bool bIsUploadComplete;

	/** To know if the ticker was started.*/
	FTSTicker::FDelegateHandle TickHandle;

};

#endif // !CRASH_REPORT_UNATTENDED_ONLY
