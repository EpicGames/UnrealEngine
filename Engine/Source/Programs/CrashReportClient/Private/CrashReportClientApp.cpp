// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportClientApp.h"
#include "CrashReportClientDefines.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/QueuedThreadPool.h"
#include "Misc/ScopeExit.h"
#include "Internationalization/Internationalization.h"
#include "Math/Vector2D.h"
#include "Misc/ConfigCacheIni.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/App.h"
#include "Misc/CString.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "CrashReportCoreConfig.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "CrashDescription.h"
#include "CrashReportAnalytics.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrashContext.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "IAnalyticsProviderET.h"
#include "XmlParser.h"
#include "Containers/Map.h"
#include "CrashReportAnalyticsSessionSummary.h"
#include "Stats/StatsSystem.h"

#if !CRASH_REPORT_UNATTENDED_ONLY
	#include "SCrashReportClient.h"
	#include "CrashReportClient.h"
	#include "CrashReportClientStyle.h"
#if !UE_BUILD_SHIPPING
	#include "ISlateReflectorModule.h"
#endif
	#include "Framework/Application/SlateApplication.h"
#endif // !CRASH_REPORT_UNATTENDED_ONLY

#include "CrashReportCoreUnattended.h"
#include "Async/TaskGraphInterfaces.h"
#include "RequiredProgramMainCPPInclude.h"

#include "MainLoopTiming.h"

#include "PlatformErrorReport.h"
#include "XmlFile.h"
#include "RecoveryService.h"

class FRecoveryService;

/** Default main window size */
const FVector2D InitialWindowDimensions(740, 560);

/** Simple dialog window size */
const FVector2D InitialSimpleWindowDimensions(740, 300);

/** Default main window size with screenshot present */
const FVector2D InitialWindowDimensionsWithScreenshot(740, 1000);

/** Simple dialog window size with screenshot present */
const FVector2D InitialSimpleWindowDimensionsWithScreenshot(920, 700);

/** Average tick rate the app aims for */
const float IdealTickRate = 30.f;

/** Set this to true in the code to open the widget reflector to debug the UI */
const bool RunWidgetReflector = false;

//WORKAROUND CL33938220. The module name is ok but the CrashReportClientEditor target is causing a warning that can be safely ignored. 
#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
#elif defined(__GNUC__)
#pragma GCC diagnostic push
#pragma GCC diagnostic ignored "-Wdeprecated-declarations"
#else
#pragma warning(push)
#pragma warning(disable : 4996) // 'function' was declared deprecated
#endif
IMPLEMENT_APPLICATION(CrashReportClient, "CrashReportClient");
#ifdef __clang__
#pragma clang diagnostic pop
#elif defined(__GNUC__)
#pragma GCC diagnostic pop
#else
#pragma warning(pop)
#endif

DEFINE_LOG_CATEGORY(CrashReportClientLog);

/** Directory containing the report */
static TArray<FString> FoundReportDirectoryAbsolutePaths;

/** Name of the game passed via the command line. */
static FString GameNameFromCmd;

/** GUID of the crash passed via the command line. */
static FString CrashGUIDFromCmd;

/** When the application invoking CRC cannot be restarted, force hiding the submit and restart button. */
static bool bForceHideSubmitAndRestartButtonFromCmd = false;

/** If we are implicitly sending its assumed we are also unattended for now */
static bool bImplicitSendFromCmd = false;
/** If we want to enable analytics */
static bool AnalyticsEnabledFromCmd = true;

/** If in monitor mode, watch this pid. */
static uint64 MonitorPid = 0;

/** If in monitor mode, pipe to read data from game. */
static void* MonitorReadPipe = nullptr;

/** If in monitor mode, pipe to write data to game. */
static void* MonitorWritePipe = nullptr;

/** If in monitor mode, set to true when the monitored app crashes. */
static bool bMonitoredAppCrashed = false;

/** In in monitor mode, this is a strong (much stronger than PID) to uniquely tie the CRC process to the monitored process. */
FString MonitorProcessGroupId;

/** Result of submission of report */
enum SubmitCrashReportResult {
	Failed,				// Failed to send report
	SuccessClosed,		// Succeeded sending report, user has not elected to relaunch
	SuccessRestarted,	// Succeeded sending report, user has elected to restart process
	SuccessContinue,	// Succeeded sending report, continue running (if monitor mode).
	SuccessDiscarded,	// User declined sending the report.
};

/**
 * Look for the report to upload, either in the command line or in the platform's report queue
 */
void ParseCommandLine(const TCHAR* CommandLine)
{
	const TCHAR* CommandLineAfterExe = FCommandLine::RemoveExeName(CommandLine);

	FoundReportDirectoryAbsolutePaths.Empty();

	// Use the first argument if present and it's not a flag
	if (*CommandLineAfterExe)
	{
		TArray<FString> Switches;
		TArray<FString> Tokens;
		TMap<FString, FString> Params;
		{
			FString NextToken;
			while (FParse::Token(CommandLineAfterExe, NextToken, false))
			{
				if (**NextToken == TCHAR('-'))
				{
					new(Switches)FString(NextToken.Mid(1));
				}
				else
				{
					new(Tokens)FString(NextToken);
				}
			}

			for (int32 SwitchIdx = Switches.Num() - 1; SwitchIdx >= 0; --SwitchIdx)
			{
				FString& Switch = Switches[SwitchIdx];
				TArray<FString> SplitSwitch;
				if (2 == Switch.ParseIntoArray(SplitSwitch, TEXT("="), true))
				{
					Params.Add(SplitSwitch[0], SplitSwitch[1].TrimQuotes());
					Switches.RemoveAt(SwitchIdx);
				}
			}
		}

		if (Tokens.Num() > 0)
		{
			FoundReportDirectoryAbsolutePaths.Add(Tokens[0]);
		}

		GameNameFromCmd = Params.FindRef(TEXT("AppName"));

		CrashGUIDFromCmd = FString();
		if (Params.Contains(TEXT("CrashGUID")))
		{
			CrashGUIDFromCmd = Params.FindRef(TEXT("CrashGUID"));
		}
 
		if (Switches.Contains(TEXT("ImplicitSend")))
		{
			bImplicitSendFromCmd = true;
		}

		if (Switches.Contains(TEXT("NoAnalytics")))
		{
			AnalyticsEnabledFromCmd = false;
		}

		if (Switches.Contains(TEXT("HideSubmitAndRestart")))
		{
			bForceHideSubmitAndRestartButtonFromCmd = true;
		}

		CrashGUIDFromCmd = Params.FindRef(TEXT("CrashGUID"));
		MonitorPid = FPlatformString::Atoi64(*Params.FindRef(TEXT("MONITOR")));
		MonitorReadPipe = (void*) FPlatformString::Atoi64(*Params.FindRef(TEXT("READ")));
		MonitorWritePipe = (void*) FPlatformString::Atoi64(*Params.FindRef(TEXT("WRITE")));
		MonitorProcessGroupId = Params.FindRef(TEXT("ProcessGroupId"));
	}

	if (FoundReportDirectoryAbsolutePaths.Num() == 0)
	{
		FPlatformErrorReport::FindMostRecentErrorReports(FoundReportDirectoryAbsolutePaths, FTimespan::FromDays(30)); //FTimespan::FromMinutes(30));
	}
}

/**
 * Find the error report folder and check it matches the app name if provided
 */
FPlatformErrorReport LoadErrorReport()
{
	if (FoundReportDirectoryAbsolutePaths.Num() == 0)
	{
		UE_LOGF(CrashReportClientLog, Warning, "No error report found");
		return FPlatformErrorReport();
	}

	for (const FString& ReportDirectoryAbsolutePath : FoundReportDirectoryAbsolutePaths)
	{
		FPlatformErrorReport ErrorReport(ReportDirectoryAbsolutePath);

		FString Filename;
		// CrashContext.runtime-xml has the precedence over the WER
		if (ErrorReport.FindFirstReportFileWithExtension(Filename, FGenericCrashContext::CrashContextExtension))
		{
			FPrimaryCrashProperties::Set(new FCrashContext(ReportDirectoryAbsolutePath / Filename));
		}
		else if (ErrorReport.FindFirstReportFileWithExtension(Filename, TEXT(".xml")))
		{
			FPrimaryCrashProperties::Set(new FCrashWERContext(ReportDirectoryAbsolutePath / Filename));
		}
		else
		{
			continue;
		}

#if CRASH_REPORT_UNATTENDED_ONLY
		return ErrorReport;
#else
		bool NameMatch = false;
		if (GameNameFromCmd.IsEmpty() || GameNameFromCmd == FPrimaryCrashProperties::Get()->GameName)
		{
			NameMatch = true;
		}

		bool GUIDMatch = false;
		if (CrashGUIDFromCmd.IsEmpty() || CrashGUIDFromCmd == FPrimaryCrashProperties::Get()->CrashGUID)
		{
			GUIDMatch = true;
		}

		if (NameMatch && GUIDMatch)
		{
			FString ConfigFilename;
			if (ErrorReport.FindFirstReportFileWithExtension(ConfigFilename, FGenericCrashContext::CrashConfigExtension))
			{
				FConfigFile CrashConfigFile;
				CrashConfigFile.Read(ReportDirectoryAbsolutePath / ConfigFilename);
				FCrashReportCoreConfig::Get().SetProjectConfigOverrides(CrashConfigFile);
			}

			return ErrorReport;
		}
#endif
	}

	// Don't display or upload anything if we can't find the report we expected
	return FPlatformErrorReport();
}

static void OnRequestExit()
{
	RequestEngineExit(TEXT("OnRequestExit"));
}

#if !CRASH_REPORT_UNATTENDED_ONLY
SubmitCrashReportResult RunWithUI(FPlatformErrorReport ErrorReport, bool bImplicitSend)
{
	// create the platform slate application (what FSlateApplication::Get() returns)
	TSharedRef<FSlateApplication> Slate = FSlateApplication::Create(MakeShareable(FPlatformApplicationMisc::CreateApplication()));

	// initialize renderer
	TSharedRef<FSlateRenderer> SlateRenderer = GetStandardStandaloneRenderer();

	// Grab renderer initialization retry settings from ini
	int32 SlateRendererInitRetryCount = 10;
	GConfig->GetInt(TEXT("CrashReportClient"), TEXT("UIInitRetryCount"), SlateRendererInitRetryCount, GEngineIni);
	double SlateRendererInitRetryInterval = 2.0;
	GConfig->GetDouble(TEXT("CrashReportClient"), TEXT("UIInitRetryInterval"), SlateRendererInitRetryInterval, GEngineIni);

	// Try to initialize the renderer. It's possible that we launched when the driver crashed so try a few times before giving up.
	bool bRendererInitialized = false;
	bool bRendererFailedToInitializeAtLeastOnce = false;
	do
	{
		SlateRendererInitRetryCount--;
		bRendererInitialized = FSlateApplication::Get().InitializeRenderer(SlateRenderer, true);
		if (!bRendererInitialized && SlateRendererInitRetryCount > 0)
		{
			bRendererFailedToInitializeAtLeastOnce = true;
			FPlatformProcess::Sleep(SlateRendererInitRetryInterval);
		}
	} while (!bRendererInitialized && SlateRendererInitRetryCount > 0);

	if (!bRendererInitialized)
	{
		// Close down the Slate application
		FSlateApplication::Shutdown();
		return Failed;
	}
	else if (bRendererFailedToInitializeAtLeastOnce)
	{
		// Wait until the driver is fully restored
		FPlatformProcess::Sleep(2.0f);

		// Update the display metrics
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		FSlateApplication::Get().GetPlatformApplication()->OnDisplayMetricsChanged().Broadcast(DisplayMetrics);
	}

	// Set up the main ticker
	FMainLoopTiming MainLoop(IdealTickRate, EMainLoopOptions::UsingSlate);

	// set the normal IsEngineExitRequested() when outer frame is closed
	FSlateApplication::Get().SetExitRequestedHandler(FSimpleDelegate::CreateStatic(&OnRequestExit));

	// Prepare the custom Slate styles.
	// If there's an optional screenshot attachment, ensure Slate loads it so that we can show it as part of the dialog.
	FCrashReportClientStyleOptions StyleOptions = {};
	FString CrashScreenshotPath = FPaths::Combine(ErrorReport.GetReportDirectory(), FGenericCrashContext::CrashReportOptionalAttachmentsDirName, TEXT("CrashCapture.jpg"));
	bool bHasScreenshot = FPaths::FileExists(CrashScreenshotPath);
	FVector2D CrashScreenshotSize{ 0, 0 };
	if (bHasScreenshot && FPrimaryCrashProperties::Get()->IsValid())
	{
		StyleOptions.OptionalCrashScreenshot.Filepath = CrashScreenshotPath;
		StyleOptions.OptionalCrashScreenshot.ImageSize = FVector2D(FPrimaryCrashProperties::Get()->CrashScreenshotWidth, FPrimaryCrashProperties::Get()->CrashScreenshotHeight);

		bHasScreenshot = !StyleOptions.OptionalCrashScreenshot.ImageSize.IsZero();
	}
	FCrashReportClientStyle::Initialize(StyleOptions);

	// Create the main implementation object
	TSharedRef<FCrashReportClient> CrashReportClient = MakeShared<FCrashReportClient>(ErrorReport, bImplicitSend);

	// Open up the app window
	// bImplicitSend now implies bSimpleDialog i.e. immediately send the report and notify the user without requesting input
	TSharedRef<SCrashReportClient> ClientControl = SNew(SCrashReportClient, CrashReportClient, bImplicitSend)
		.bHideSubmitAndRestart(bForceHideSubmitAndRestartButtonFromCmd);

	FString CrashedAppName = FPrimaryCrashProperties::Get()->IsValid() ? FPrimaryCrashProperties::Get()->GameName : TEXT("");
	// GameNames have taken on a number of prefixes over the years. Try to strip them all off.
	if (!CrashedAppName.RemoveFromStart(TEXT("UE4-")))
	{
		if (!CrashedAppName.RemoveFromStart(TEXT("UE5-")))
		{
			CrashedAppName.RemoveFromStart(TEXT("UE-"));
		}
	}
	CrashedAppName.RemoveFromEnd(TEXT("Game"));

	const FString CrashedAppString = NSLOCTEXT("CrashReportClient", "CrashReporterTitle", "Crash Reporter").ToString();
	const FText CrashedAppText = FText::FromString(FString::Printf(TEXT("%s %s"), *CrashedAppName, *CrashedAppString));

	// Get the engine major version to display in title.
	FBuildVersion BuildVersion;
	uint16 MajorEngineVersion = FBuildVersion::TryRead(FBuildVersion::GetDefaultFileName(), BuildVersion) ? BuildVersion.GetEngineVersion().GetMajor() : 5;

	FText WindowTitle = CrashedAppName.IsEmpty() ?
		FText::Format(NSLOCTEXT("CrashReportClient", "CrashReportClientAppName", "Unreal Engine {0} Crash Reporter"), MajorEngineVersion) :
		CrashedAppText;

	FVector2D WindowDimensions;
	if (bImplicitSend)
	{
		WindowDimensions = bHasScreenshot ? InitialSimpleWindowDimensionsWithScreenshot : InitialSimpleWindowDimensions;
	}
	else
	{
		WindowDimensions = bHasScreenshot ? InitialWindowDimensionsWithScreenshot : InitialWindowDimensions;
	}

	TSharedRef<SWindow> Window = FSlateApplication::Get().AddWindow(
		SNew(SWindow)
		.Title(WindowTitle)
		.HasCloseButton(FCrashReportCoreConfig::Get().IsAllowedToCloseWithoutSending())
		.ClientSize(WindowDimensions)
		[
			ClientControl
		]);

	Window->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateSP(CrashReportClient, &FCrashReportClient::RequestCloseWindow));

	// Setting focus seems to have to happen after the Window has been added
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);

#if !UE_BUILD_SHIPPING
	// Debugging code
	if (RunWidgetReflector)
	{
		FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector").DisplayWidgetReflector();
	}
#endif

	//
	// The Mac implementation of the window class did not implement HACK_ForceToFront().
	// In order to patch a CRC visiblity issue without breaking binary compatibility on
	// the Mac, as well as not changing the behavior on other platforms, we explicity
	// pass in the force flag on that platform only.
	//
	// TODO: Implement HACK_ForceToFront() for macOS and remove bForceBringToFront from here.
	//
	const bool bForceBringToFront = (false || (PLATFORM_MAC));

	// Bring the window to the foreground as it may be behind the crashed process
	Window->HACK_ForceToFront();
	Window->BringToFront(bForceBringToFront);

	// loop until the app is ready to quit
	while (!(IsEngineExitRequested() || ClientControl->IsFinished()))
	{
		MainLoop.Tick();

		if (CrashReportClient->ShouldWindowBeHidden())
		{
			Window->HideWindow();
		}
	}

	// Make sure the window is hidden, because it might take a while for the background thread to finish.
	Window->HideWindow();

	// Stop the background thread
	CrashReportClient->StopBackgroundThread();

	// Clean up the custom styles
	FCrashReportClientStyle::Shutdown();

	// Close down the Slate application
	FSlateApplication::Shutdown();

	// Detect if ensure, if user has selected to restart or close.
	if (CrashReportClient->WasClosedWithoutSending())
	{
		return SuccessDiscarded;
	}
	else if (CrashReportClient->IsUploadComplete())
	{
		return CrashReportClient->GetIsSuccesfullRestart() ? SuccessRestarted : (FPrimaryCrashProperties::Get()->bIsEnsure ? SuccessContinue : SuccessClosed);
	}
	
	return Failed;
}
#endif // !CRASH_REPORT_UNATTENDED_ONLY

// When we want to implicitly send and use unattended we still want to show a message box of a crash if possible
class FMessageBoxThread : public FRunnable
{
	virtual uint32 Run() override
	{
		// We will not have any GUI for the crash reporter if we are sending implicitly, so pop a message box up at least
		if (FApp::CanEverRender() && !FApp::IsUnattended())
		{
			FString Body = *NSLOCTEXT("MessageDialog", "ReportCrash_Body", "The application has crashed and will now close. We apologize for the inconvenience.").ToString();
			if (FPrimaryCrashProperties::Get()->IsValid())
			{
				Body = FPrimaryCrashProperties::Get()->CrashReporterMessage.AsString();
			}

			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok,
				*Body,
				*NSLOCTEXT("MessageDialog", "ReportCrash_Title", "Application Crash Detected").ToString());
		}

		return 0;
	}
};

SubmitCrashReportResult RunUnattended(FPlatformErrorReport ErrorReport, bool bImplicitSend)
{
	// Set up the main ticker
	FMainLoopTiming MainLoop(IdealTickRate, EMainLoopOptions::CoreTickerOnly);

	// In the unattended mode we don't send any PII.
	FCrashReportCoreUnattended CrashReportClient(ErrorReport);
	ErrorReport.SetUserComment(NSLOCTEXT("CrashReportClient", "UnattendedMode", "Sent in the unattended mode"));

	FMessageBoxThread MessageBox;
	FRunnableThread* MessageBoxThread = nullptr;

	if (bImplicitSend)
	{
		MessageBoxThread = FRunnableThread::Create(&MessageBox, TEXT("CrashReporter_MessageBox"));
	}

	// loop until the app is ready to quit
	while (!(IsEngineExitRequested() || CrashReportClient.IsUploadComplete()))
	{
		MainLoop.Tick();
	}

	if (bImplicitSend && MessageBoxThread)
	{
		MessageBoxThread->WaitForCompletion();
	}

	// Continue running in case of ensures, otherwise close
	return FPrimaryCrashProperties::Get()->bIsEnsure ? SuccessContinue : SuccessClosed;
}

FPlatformErrorReport CollectErrorReport(FRecoveryService* RecoveryService, uint32 Pid, const FSharedCrashContext& SharedCrashContext, void* WritePipe, bool& bOutCrashPortableCallstackAvailable)
{
	bOutCrashPortableCallstackAvailable = false;

	// @note: This API is only partially implemented on Mac OS and Linux.
	FProcHandle ProcessHandle = FPlatformProcess::OpenProcess(Pid);
	if (!ProcessHandle.IsValid())
	{
		FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Report/OpenProcessFail"));
	}
	else if (SharedCrashContext.CrashingThreadId == 0)
	{
		FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Report/BadCrashThreadId"));
	}
	else if (SharedCrashContext.NumThreads == CR_MAX_THREADS)
	{
		FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Report/BumpThreadLimits"));
	}

	// First init the static crash context state
	FPlatformCrashContext::InitializeFromContext(
		SharedCrashContext.SessionContext,
		SharedCrashContext.EnabledPluginsNum > 0 ? &SharedCrashContext.DynamicData[SharedCrashContext.EnabledPluginsOffset] : nullptr,
		SharedCrashContext.EngineDataNum > 0 ? &SharedCrashContext.DynamicData[SharedCrashContext.EngineDataOffset] : nullptr,
		SharedCrashContext.GameDataNum > 0 ? &SharedCrashContext.DynamicData[SharedCrashContext.GameDataOffset] : nullptr,
		SharedCrashContext.FilePathsNum > 0 ? &SharedCrashContext.DynamicData[SharedCrashContext.FilePathsOffset] : nullptr,
		&SharedCrashContext.GPUBreadcrumbs
	);
	// Next create a crash context for the crashed process.
	FPlatformCrashContext CrashContext(SharedCrashContext.CrashType, SharedCrashContext.ErrorMessage);
	CrashContext.SetCrashedProcess(ProcessHandle);
	CrashContext.SetCrashedThreadId(SharedCrashContext.CrashingThreadId);
	CrashContext.SetNumMinidumpFramesToIgnore(0);

	FCrashReportAnalyticsSessionSummary::Get().OnCrashReportRemoteStackWalking();

	// Initialize the stack walking for the monitored process (effectively overriding this process stack walking functionality)
	FPlatformStackWalk::InitStackWalkingForProcess(ProcessHandle);

	TArray<TArray<uint64>> ThreadCallStacks;
	ThreadCallStacks.Reserve(SharedCrashContext.NumThreads);
	for (uint32 ThreadIdx = 0; ThreadIdx < SharedCrashContext.NumThreads; ThreadIdx++)
	{
		const uint32 ThreadId = SharedCrashContext.ThreadIds[ThreadIdx];
		TSharedPtr<void> PlatformContext;

#if PLATFORM_WINDOWS
		// This code let us acquire the complete portable callstack of the remote process after it crashed on a null pointer function invokation. To successfully walk the
		// stack where a null pointer function is called, we need to provide the thread context reported in the crash (the pointer passed to minidump function), otherwise,
		// the portable callstack is incomplete.
		if (ThreadId == SharedCrashContext.CrashingThreadId)
		{
			SIZE_T ReadCount = 0;

			// On Windows, 'PlatformCrashContext' is a pointer to the EXCEPTION_POINTERS struct. Try to read it from the monitored process memory.
			EXCEPTION_POINTERS ExceptPtrs{nullptr, nullptr};
			if (::ReadProcessMemory(ProcessHandle.Get(), SharedCrashContext.PlatformCrashContext, &ExceptPtrs, sizeof(EXCEPTION_POINTERS), &ReadCount) && ReadCount == sizeof(EXCEPTION_POINTERS))
			{
				// Try to read memory of the CONTEXT member from the monitored process.
				CONTEXT WindowsContext;
				FMemory::Memzero(WindowsContext);
				if (::ReadProcessMemory(ProcessHandle.Get(), ExceptPtrs.ContextRecord, &WindowsContext, sizeof(CONTEXT), &ReadCount) && ReadCount == sizeof(CONTEXT))
				{
					// NOTE: CaptureThreadStackBackTrace() will open and supply the thread handle specified as null here.
					PlatformContext = TSharedPtr<void>(FWindowsPlatformStackWalk::MakeThreadContextWrapper(&WindowsContext, nullptr), [](void* Ptr)
					{
						FWindowsPlatformStackWalk::ReleaseThreadContextWrapper(Ptr);
					});
				}
			}
		}
#endif

		uint64 StackFrames[CR_MAX_STACK_FRAMES] = {0};
		uint32 StackFrameCount = FPlatformStackWalk::CaptureThreadStackBackTrace(
			ThreadId, 
			StackFrames,
			CR_MAX_STACK_FRAMES,
			PlatformContext.Get()
		);
		
		ThreadCallStacks.Emplace(TArray<uint64>(StackFrames, StackFrameCount));		

		// CrashContext.AddPortableThreadCallStack(
		// 	SharedCrashContext.ThreadIds[ThreadIdx],
		// 	&SharedCrashContext.ThreadNames[ThreadIdx*CR_MAX_THREAD_NAME_CHARS],
		// 	StackFrames,
		// 	StackFrameCount
		// );

		// Add the crashing stack specifically. Is this really needed?
		if (ThreadId == SharedCrashContext.CrashingThreadId)
		{
			const uint64* StackFrameCursor = StackFrames;

			// If the address of where the error occurred has been provided then
			// we can remove the boilerplate noise from the callstack.
			if (uint64 ErrorPc = uint64(SharedCrashContext.ErrorProgramCounter))
			{
				uint64 ExceptionPc = uint64(SharedCrashContext.ExceptionProgramCounter);
				int32 ExceptionDepth = -1;
				for (uint32 i = 0; i < StackFrameCount; ++i)
				{
					if (StackFrames[i] == ExceptionPc)
					{
						ExceptionDepth = i;
					}

					if (StackFrames[i] != ErrorPc)
					{
						continue;
					}

					if (ExceptionDepth >= 0)
					{
						CrashContext.SetNumMinidumpFramesToIgnore(i - ExceptionDepth);
					}

					StackFrameCursor = StackFrames + i;
					StackFrameCount -= i;
					break;
				}
			}

			CrashContext.SetPortableCallStack(StackFrameCursor, StackFrameCount);

			// A completely missing portable callstack usually means that the crashing process died before CRC could walk the stack.
			bOutCrashPortableCallstackAvailable = StackFrameCount > 0;
		}
	}

	{
		TArray<FThreadCallStack> Threads;
		Threads.Reserve(SharedCrashContext.NumThreads);
		for (uint32 ThreadIdx = 0; ThreadIdx < SharedCrashContext.NumThreads; ++ThreadIdx)
		{
			Threads.Add({
				MakeArrayView(ThreadCallStacks[ThreadIdx]),
				&SharedCrashContext.ThreadNames[ThreadIdx*CR_MAX_THREAD_NAME_CHARS],
				SharedCrashContext.ThreadIds[ThreadIdx],
			});
		}
		CrashContext.AddPortableThreadCallStacks(Threads);
	}


	FCrashReportAnalyticsSessionSummary::Get().OnCrashReportGatheringFiles();

	// If the path is not set it is most likely that we have crashed during static init, in which case we need to construct a directory ourself.
	FString ReportDirectoryAbsolutePath(SharedCrashContext.CrashFilesDirectory);
	bool DirectoryExists = true;
	if (ReportDirectoryAbsolutePath.IsEmpty())
	{
		DirectoryExists = FGenericCrashContext::CreateCrashReportDirectory(
			SharedCrashContext.SessionContext.CrashGUIDRoot,
			0,
			ReportDirectoryAbsolutePath);
	}

	// Copy platform specific files (e.g. minidump) to output directory if it exists
	if (DirectoryExists)
	{
		CrashContext.CopyPlatformSpecificFiles(*ReportDirectoryAbsolutePath, SharedCrashContext.PlatformCrashContext);
	}

	FCrashReportAnalyticsSessionSummary::Get().OnCrashReportSignalingAppToResume();

	// At this point the game can continue execution. It is important this happens
	// as soon as thread state and minidump has been created, so that ensures cause
	// as little hitch as possible.
	uint8 ResponseCode[] = { 0xd, 0xe, 0xa, 0xd };
	FPlatformProcess::WritePipe(WritePipe, ResponseCode, sizeof(ResponseCode));

	// Write out the XML file.
	const FString CrashContextXMLPath = FPaths::Combine(*ReportDirectoryAbsolutePath, FPlatformCrashContext::CrashContextRuntimeXMLNameW);
	CrashContext.SerializeAsXML(*CrashContextXMLPath);

#if CRASH_REPORT_WITH_RECOVERY
	if (RecoveryService && 
		DirectoryExists && 
		SharedCrashContext.UserSettings.bSendUsageData && 
		!FPlatformCrashContext::IsTypeContinuable(SharedCrashContext.CrashType)
	{
		RecoveryService->CollectFiles(ReportDirectoryAbsolutePath);
	}
#endif

	// If the crash context wasn't implicitely serialized by SerializeAsXML() above, serialize it now.
	if (CrashContext.GetBuffer().IsEmpty())
	{
		CrashContext.SerializeContentToBuffer();
	}

	// Setup the FPrimaryCrashProperties singleton.
	const TCHAR* CrashContextBuffer = *CrashContext.GetBuffer();
	FPrimaryCrashProperties::Set(new FCrashContext(ReportDirectoryAbsolutePath / TEXT("CrashContext.runtime-xml"), CrashContextBuffer));

	FPlatformErrorReport ErrorReport(ReportDirectoryAbsolutePath);

	// Link the crash to the Editor summary event to help diagnose the abnormal termination quickly.
	FCrashReportAnalyticsSessionSummary::Get().LogEvent(*FPrimaryCrashProperties::Get()->CrashGUID);

	// Reset stack walking to allow CRC to implicitly walk its own process and close the monitored process handle.
	FPlatformStackWalk::InitStackWalkingForProcess(FProcHandle());
	FPlatformProcess::CloseProc(ProcessHandle);

#if CRASH_REPORT_UNATTENDED_ONLY
	return ErrorReport;
#else

	FString ConfigFilename;
	if (ErrorReport.FindFirstReportFileWithExtension(ConfigFilename, FGenericCrashContext::CrashConfigExtension))
	{
		FConfigFile CrashConfigFile;
		CrashConfigFile.Read(ReportDirectoryAbsolutePath / ConfigFilename);
		FCrashReportCoreConfig::Get().SetProjectConfigOverrides(CrashConfigFile);
	}

	return ErrorReport;
#endif
}

SubmitCrashReportResult SendErrorReport(FPlatformErrorReport& ErrorReport, 
	TOptional<bool> bNoDialogOpt = TOptional<bool>(), 
	TOptional<bool> bImplicitSendOpt = TOptional<bool>())
{
	if (!IsEngineExitRequested() && ErrorReport.HasFilesToUpload() && FPrimaryCrashProperties::Get() != nullptr)
	{
		const bool bImplicitSend = bImplicitSendOpt.Get(false);
		const bool bUnattended = CRASH_REPORT_UNATTENDED_ONLY ? true : bNoDialogOpt.Get(FApp::IsUnattended());

		ErrorReport.SetCrashReportClientVersion(FCrashReportCoreConfig::Get().GetVersion());

		if (bUnattended)
		{
			return RunUnattended(ErrorReport, bImplicitSend);
		}
#if !CRASH_REPORT_UNATTENDED_ONLY
		else
		{
			const SubmitCrashReportResult Result = RunWithUI(ErrorReport, bImplicitSend);
			if (Result == Failed)
			{
				// UI failed to initialize, probably due to driver crash. Send in unattended mode if allowed.
				bool bCanSendWhenUIFailedToInitialize = true;
				GConfig->GetBool(TEXT("CrashReportClient"), TEXT("CanSendWhenUIFailedToInitialize"), bCanSendWhenUIFailedToInitialize, GEngineIni);
				if (bCanSendWhenUIFailedToInitialize && !FCrashReportCoreConfig::Get().IsAllowedToCloseWithoutSending())
				{
					return RunUnattended(ErrorReport, bImplicitSend);
				}
			}
			return Result;
		}
#endif // !CRASH_REPORT_UNATTENDED_ONLY

	}
	return Failed;
}

bool IsCrashReportAvailable(uint32 WatchedProcess, FSharedCrashContext& CrashContext, void* ReadPipe)
{
	TArray<uint8> Buffer;

	// Is data available on the pipe.
	if (FPlatformProcess::ReadPipeToArray(ReadPipe, Buffer))
	{
		FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/Read"));

		// This is to ensure the FSharedCrashContext compiled in the monitored process and this process has the same size.
		int32 TotalRead = Buffer.Num();

		// Utility function to copy bytes from a source to a destination buffer.
		auto CopyFn = [](const TArray<uint8>& SrcData, uint8* DstIt, uint8* DstEndIt)
		{
			int32 CopyCount = FMath::Min(SrcData.Num(), static_cast<int32>(DstEndIt - DstIt)); // Limit the number of byte to copy to avoid writing passed the end of the destination.
			FPlatformMemory::Memcpy(DstIt, SrcData.GetData(), CopyCount);
			return DstIt + CopyCount; // Returns the updated position.
		};

		// Iterators to defines the boundaries of the destination buffer in memory.
		uint8* SharedCtxIt = reinterpret_cast<uint8*>(&CrashContext);
		uint8* SharedCtxEndIt = SharedCtxIt + sizeof(FSharedCrashContext);

		// Copy the data already read and update the destination iterator.
		SharedCtxIt = CopyFn(Buffer, SharedCtxIt, SharedCtxEndIt);

		// Try to consume all the expected data within a defined period of time.
		double WaitEndTime = FPlatformTime::Seconds() + 5;
		while (SharedCtxIt != SharedCtxEndIt && FPlatformTime::Seconds() <= WaitEndTime)
		{
			if (FPlatformProcess::ReadPipeToArray(ReadPipe, Buffer)) // This is false if no data is available, but the writer may be still be writing.
			{
				TotalRead += Buffer.Num();
				SharedCtxIt = CopyFn(Buffer, SharedCtxIt, SharedCtxEndIt); // Copy the data read.
			}
			else
			{
				FPlatformProcess::Sleep(0.1f); // Give the writer some time.
			}
		}

		if (TotalRead < sizeof(FSharedCrashContext))
		{
			FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/NotEnoughData"));
		}
		else if (TotalRead > sizeof(FSharedCrashContext))
		{
			FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/TooMuchData"));
		}
		else
		{
			// Record the history of events sent by the Editor to help diagnose abnormal terminations.
			switch (CrashContext.CrashType)
			{
				case ECrashContextType::Assert:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/Assert"));
					bMonitoredAppCrashed = true;
					break;

				case ECrashContextType::Ensure:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/Ensure"));
					break;

				case ECrashContextType::Stall:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/Stall"));
					break;

				case ECrashContextType::Crash:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/Crash"));
					bMonitoredAppCrashed = true;
					break;

				case ECrashContextType::GPUCrash:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/GPUCrash"));
					bMonitoredAppCrashed = true;
					break;

				case ECrashContextType::Hang:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/Hang"));
					break;

				case ECrashContextType::OutOfMemory:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/OOM"));
					bMonitoredAppCrashed = true;
					break;

				case ECrashContextType::AbnormalShutdown:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/AbnormalShutdown"));
					bMonitoredAppCrashed = true;
					break;

				default:
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Pipe/Unknown"));
					break;
			}
		}

		return SharedCtxIt == SharedCtxEndIt;
	}

	return false;
}

static void DeleteTempCrashContextFile(uint64 ProcessID)
{
	const FString SessionContextFile = FGenericCrashContext::GetTempSessionContextFilePath(ProcessID);
	FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*SessionContextFile);
}

static void DeleteExpiredTempCrashContext(const FTimespan& ExpirationAge)
{
	// Clean up old temp context that were likely left over by crashed/killed CRC (unless the process that wrote it is still alive).
	// If by any chances a CRC instance deletes the context that was produced for another CRC instance that is still running after the
	// expiration delay, then we may lose the analytics session and not spoof an abnormal shutdown. In general, the consequences are none
	// to meaningless, so we are better keeping the user disk clean.
	FGenericCrashContext::CleanupTempSessionContextFiles(ExpirationAge);
}

#if CRASH_REPORT_WITH_MTBF

template <typename Type>
bool FindAndParseValue(const TMap<FString, FString>& Map, const FString& Key, Type& OutValue)
{
	const FString* ValueString = Map.Find(Key);
	if (ValueString != nullptr)
	{
		TTypeFromString<Type>::FromString(OutValue, **ValueString);
		return true;
	}

	return false;
}

template <size_t Size>
bool FindAndCopyValue(const TMap<FString, FString>& Map, const FString& Key, TCHAR (&OutValue)[Size])
{
	const FString* ValueString = Map.Find(Key);
	if (ValueString != nullptr)
	{
		FCString::Strncpy(OutValue, **ValueString, Size);
		return true;
	}

	return false;
}

static bool LoadTempCrashContextFromFile(FSharedCrashContext& CrashContext, uint64 ProcessID)
{
	const FString TempContextFilePath = FGenericCrashContext::GetTempSessionContextFilePath(ProcessID);

	FXmlFile File;
	if (!File.LoadFile(TempContextFilePath))
	{
		return false;
	}

	TMap<FString, FString> ContextProperties;
	for (FXmlNode* Node : File.GetRootNode()->GetChildrenNodes())
	{
		ContextProperties.Add(Node->GetTag(), Node->GetContent());
	}

	FSessionContext& SessionContext = CrashContext.SessionContext;

	FindAndParseValue(ContextProperties, TEXT("SecondsSinceStart"), SessionContext.SecondsSinceStart);
	FindAndParseValue(ContextProperties, TEXT("IsInternalBuild"), SessionContext.bIsInternalBuild);
	FindAndParseValue(ContextProperties, TEXT("IsPerforceBuild"), SessionContext.bIsPerforceBuild);
	FindAndParseValue(ContextProperties, TEXT("IsSourceDistribution"), SessionContext.bIsSourceDistribution);
	FindAndCopyValue(ContextProperties, TEXT("GameName"), SessionContext.GameName);
	FindAndCopyValue(ContextProperties, TEXT("ExecutableName"), SessionContext.ExecutableName);
	FindAndCopyValue(ContextProperties, TEXT("BuildConfiguration"), SessionContext.BuildConfigurationName);
	FindAndCopyValue(ContextProperties, TEXT("GameSessionID"), SessionContext.GameSessionID);
	FindAndCopyValue(ContextProperties, TEXT("PlatformName"), SessionContext.PlatformName);
	FindAndCopyValue(ContextProperties, TEXT("PlatformNameIni"), SessionContext.PlatformNameIni);
	FindAndCopyValue(ContextProperties, TEXT("EngineMode"), SessionContext.EngineMode);
	FindAndCopyValue(ContextProperties, TEXT("EngineModeEx"), SessionContext.EngineModeEx);
	FindAndCopyValue(ContextProperties, TEXT("DeploymentName"), SessionContext.DeploymentName);
	FindAndCopyValue(ContextProperties, TEXT("EngineVersion"), SessionContext.EngineVersion);
	FindAndCopyValue(ContextProperties, TEXT("EngineCompatibleVersion"), SessionContext.EngineCompatibleVersion);
	FindAndCopyValue(ContextProperties, TEXT("CommandLine"), SessionContext.CommandLine);
	FindAndParseValue(ContextProperties, TEXT("LanguageLCID"), SessionContext.LanguageLCID);
	FindAndCopyValue(ContextProperties, TEXT("AppDefaultLocale"), SessionContext.DefaultLocale);
	FindAndCopyValue(ContextProperties, TEXT("BuildVersion"), SessionContext.BuildVersion);
	FindAndParseValue(ContextProperties, TEXT("IsUERelease"), SessionContext.bIsUERelease);
	FindAndCopyValue(ContextProperties, TEXT("UserName"), SessionContext.UserName);
	FindAndCopyValue(ContextProperties, TEXT("EpicAccountId"), SessionContext.EpicAccountId);
	FindAndCopyValue(ContextProperties, TEXT("BaseDir"), SessionContext.BaseDir);
	FindAndCopyValue(ContextProperties, TEXT("RootDir"), SessionContext.RootDir);
	FindAndCopyValue(ContextProperties, TEXT("LoginId"), SessionContext.LoginIdStr);
	FindAndCopyValue(ContextProperties, TEXT("EpicAccountId"), SessionContext.EpicAccountId);
	FindAndCopyValue(ContextProperties, TEXT("UserActivityHint"), SessionContext.UserActivityHint);
	FindAndParseValue(ContextProperties, TEXT("CrashDumpMode"), SessionContext.CrashDumpMode);
	FindAndCopyValue(ContextProperties, TEXT("GameStateName"), SessionContext.GameStateName);
	FindAndParseValue(ContextProperties, TEXT("Misc.NumberOfCores"), SessionContext.NumberOfCores);
	FindAndParseValue(ContextProperties, TEXT("Misc.NumberOfCoresIncludingHyperthreads"), SessionContext.NumberOfCoresIncludingHyperthreads);
	FindAndCopyValue(ContextProperties, TEXT("Misc.CPUVendor"), SessionContext.CPUVendor);
	FindAndCopyValue(ContextProperties, TEXT("Misc.CPUBrand"), SessionContext.CPUBrand);
	FindAndCopyValue(ContextProperties, TEXT("Misc.PrimaryGPUBrand"), SessionContext.PrimaryGPUBrand);
	FindAndCopyValue(ContextProperties, TEXT("Misc.OSVersionMajor"), SessionContext.OsVersion);
	FindAndCopyValue(ContextProperties, TEXT("Misc.OSVersionMinor"), SessionContext.OsSubVersion);
	FindAndCopyValue(ContextProperties, TEXT("Misc.AnticheatProvider"), SessionContext.AnticheatProvider);
	FindAndParseValue(ContextProperties, TEXT("Misc.IsStuck"), SessionContext.bIsStuck);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.AvailablePhysical"), SessionContext.MemoryStats.AvailablePhysical);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.AvailableVirtual"), SessionContext.MemoryStats.AvailableVirtual);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.UsedPhysical"), SessionContext.MemoryStats.UsedPhysical);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.PeakUsedPhysical"), SessionContext.MemoryStats.PeakUsedPhysical);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.UsedVirtual"), SessionContext.MemoryStats.UsedVirtual);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.PeakUsedVirtual"), SessionContext.MemoryStats.PeakUsedVirtual);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.bIsOOM"), SessionContext.bIsOOM);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.OOMAllocationSize"), SessionContext.OOMAllocationSize);
	FindAndParseValue(ContextProperties, TEXT("MemoryStats.OOMAllocationAlignment"), SessionContext.OOMAllocationAlignment);

	// user settings
	FUserSettingsContext& UserSettings = CrashContext.UserSettings;

	FindAndParseValue(ContextProperties, TEXT("NoDialog"), UserSettings.bNoDialog);
	FindAndParseValue(ContextProperties, TEXT("SendUnattendedBugReports"), UserSettings.bSendUnattendedBugReports);
	FindAndParseValue(ContextProperties, TEXT("SendUsageData"), UserSettings.bSendUsageData);
	FindAndCopyValue(ContextProperties, TEXT("LogFilePath"), UserSettings.LogFilePath);

	// Dynamic data

	// When encoding the plugins list and engine and game data key/value pairs into the dynamic data segment
	// we use 1 and 2 to denote delimiter and equals respectively. This is necessary since the values could 
	// contain any characters normally used for delimiting.
	const TCHAR* CR_PAIR_DELIM = TEXT("\x01");
	const TCHAR* CR_PAIR_EQ = TEXT("\x02");

	TStringBuilder<0> DynamicDataBuilder;
	DynamicDataBuilder.Reserve(CR_MAX_DYNAMIC_BUFFER_CHARS);

	// Enabled Plugins
	CrashContext.EnabledPluginsOffset = (uint32)DynamicDataBuilder.Len();
	if (const FXmlNode* EnabledPluginsNode = File.GetRootNode()->FindChildNode(TEXT("EnabledPlugins")))
	{
		const TArray<FXmlNode*>& EnabledPlugins = EnabledPluginsNode->GetChildrenNodes();
		CrashContext.EnabledPluginsNum = EnabledPlugins.Num();
		for (const FXmlNode* Node : EnabledPlugins)
		{
			DynamicDataBuilder.Append(Node->GetContent());
			DynamicDataBuilder.Append(CR_PAIR_DELIM);
		}
	}
	else
	{
		CrashContext.EnabledPluginsNum = 0;
	}
	
	// Engine Data
	CrashContext.EngineDataOffset = (uint32)DynamicDataBuilder.Len();
	if (const FXmlNode* EngineDataNode = File.GetRootNode()->FindChildNode(TEXT("EngineData")))
	{
		const TArray<FXmlNode*>& EngineDataEntries = EngineDataNode->GetChildrenNodes();
		CrashContext.EngineDataNum = EngineDataEntries.Num();
		for (const FXmlNode* Node : EngineDataEntries)
		{
			DynamicDataBuilder.Append(Node->GetTag());
			DynamicDataBuilder.Append(CR_PAIR_EQ);
			DynamicDataBuilder.Append(Node->GetContent());
			DynamicDataBuilder.Append(CR_PAIR_DELIM);
		}
	}
	else
	{
		CrashContext.EngineDataNum = 0;
	}
	
	// Game Data
	CrashContext.GameDataOffset = (uint32)DynamicDataBuilder.Len();
	if (const FXmlNode* GameDataNode = File.GetRootNode()->FindChildNode(TEXT("GameData")))
	{
		const TArray<FXmlNode*>& GameDataEntries = GameDataNode->GetChildrenNodes();
		CrashContext.GameDataNum = GameDataEntries.Num();
		for (const FXmlNode* Node : GameDataEntries)
		{
			DynamicDataBuilder.Append(Node->GetTag());
			DynamicDataBuilder.Append(CR_PAIR_EQ);
			DynamicDataBuilder.Append(Node->GetContent());
			DynamicDataBuilder.Append(CR_PAIR_DELIM);
		}
	}
	else
	{
		CrashContext.GameDataNum = 0;
	}
	
	// File Paths
	CrashContext.FilePathsOffset = (uint32)DynamicDataBuilder.Len();
	if (const FXmlNode* FilePathsNode = File.GetRootNode()->FindChildNode(TEXT("FilePaths")))
	{
		const TArray<FXmlNode*>& FilePaths = FilePathsNode->GetChildrenNodes();
		CrashContext.FilePathsNum = FilePaths.Num();
		for (const FXmlNode* Node : FilePaths)
		{
			DynamicDataBuilder.Append(Node->GetContent());
			DynamicDataBuilder.Append(CR_PAIR_DELIM);
		}
	}
	else
	{
		CrashContext.FilePathsNum = 0;
	}

	FCString::Strncpy(CrashContext.DynamicData, DynamicDataBuilder.ToString(), CR_MAX_DYNAMIC_BUFFER_CHARS - 1);
	CrashContext.DynamicData[CR_MAX_DYNAMIC_BUFFER_CHARS - 1] = 0;

	return true;
}

FString FormatExitCode(int32 ExitCode)
{
	// Translate common exit codes.
	auto GetExitCodeName = [](int32 Code) -> const TCHAR*
	{
#if PLATFORM_WINDOWS
		switch (Code)
		{
			case  1073807364: return TEXT("DBG_TERMINATE_PROCESS"); // Typically when the user logs out or the system is shutting down.
			case -1073740286: return TEXT("STATUS_FAIL_FAST_EXCEPTION");
			case -1073740771: return TEXT("STATUS_FATAL_USER_CALLBACK_EXCEPTION");
			case -1073740791: return TEXT("STATUS_STACK_BUFFER_OVERRUN");
			case -1073740940: return TEXT("STATUS_HEAP_CORRUPTION");
			case -1073741395: return TEXT("STATUS_FATAL_MEMORY_EXHAUSTION");
			case -1073741510: return TEXT("STATUS_CONTROL_C_EXIT");
			case -1073741571: return TEXT("STATUS_STACK_OVERFLOW");
			case -1073741676: return TEXT("STATUS_INTEGER_DIVIDE_BY_ZERO");
			case -1073741795: return TEXT("STATUS_ILLEGAL_INSTRUCTION");
			case -1073741811: return TEXT("STATUS_INVALID_PARAMETER");
			case -1073741818: return TEXT("STATUS_IN_PAGE_ERROR");
			case -1073741819: return TEXT("STATUS_ACCESS_VIOLATION");
			default:          return nullptr;
		}
#else
		return nullptr;
#endif
	};

	const TCHAR* ExitCodeName = GetExitCodeName(ExitCode);
	if (ExitCodeName)
	{
		return FString::Printf(TEXT("%d (%s)"), ExitCode, ExitCodeName);
	}
	return LexToString(ExitCode);
}

static void HandleAbnormalShutdown(FSharedCrashContext& CrashContext, uint64 ProcessID, void* WritePipe, const TSharedPtr<FRecoveryService>& RecoveryService, const TOptional<int32>& ExitCode)
{
	CrashContext.CrashType = ECrashContextType::AbnormalShutdown;
	if (ExitCode.IsSet())
	{
		// Set the error message like: AbnormalShutdown - ExitCode: -1073741571 (STATUS_STACK_OVERFLOW)
		FCString::Sprintf(CrashContext.ErrorMessage, TEXT("AbnormalShutdown - ExitCode: %s"), *FormatExitCode(*ExitCode));
	}
	else
	{
		FCString::Strcpy(CrashContext.ErrorMessage, TEXT("AbnormalShutdown"));
	}

	// Normally, the CrashGUIDRoot is generated by the Editor/Engine and a counter is appended to it. Starting at zero, the counter is incremented after each ensure/crash by the Editor/Engine.
	// In this cases, the crash doesn't originate from the Editor/Engine, but CRC. The Editor/Engine CrashGUIDRoot isn't serialized in the temp context file so we need to generate  a new one.
	// This also ensure we don't collide with the one emitted by the Editor as the counter part in this process would also start at zero.
	FGuid CrashGUID = FGuid::NewGuid();
	FString IniPlatformName(FPlatformProperties::IniPlatformName()); // To convert from char* to TCHAR*
	FCString::Strcpy(CrashContext.SessionContext.CrashGUIDRoot, *FString::Printf(TEXT("%s%s-%s"), FGenericCrashContext::CrashGUIDRootPrefix, *IniPlatformName, *CrashGUID.ToString(EGuidFormats::Digits)));

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	// create a temporary crash directory
	const FString TempCrashDirectory = FPlatformProcess::UserTempDir() / FString::Printf(TEXT("UECrashContext-%lld"), ProcessID);
	FCString::Strcpy(CrashContext.CrashFilesDirectory, *TempCrashDirectory);

	if (PlatformFile.CreateDirectory(CrashContext.CrashFilesDirectory))
	{
		// copy the log file to the temporary directory
		const FString LogDestination = TempCrashDirectory / FPaths::GetCleanFilename(CrashContext.UserSettings.LogFilePath);
		PlatformFile.CopyFile(*LogDestination, CrashContext.UserSettings.LogFilePath);

		// This crash is not a real one, but one to capture the Editor logs in case of abnormal termination.
		FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("SyntheticCrash"));

		bool bPortableCallstackAvailable = false; // Should always be false here. The process is dead.
		FPlatformErrorReport ErrorReport = CollectErrorReport(RecoveryService.Get(), ProcessID, CrashContext, WritePipe, bPortableCallstackAvailable);
		SendErrorReport(ErrorReport, /*bNoDialog*/ true);

		// delete the temporary directory
		PlatformFile.DeleteDirectoryRecursively(*TempCrashDirectory);

		if (CrashContext.UserSettings.bSendUsageData)
		{
			// If analytics is enabled make sure they are submitted now.
			FCrashReportAnalytics::GetProvider().BlockUntilFlushed(5.0f);
		}
	}
}

#endif

void RunCrashReportClient(const TCHAR* CommandLine)
{
	FTaskTagScope ThreadScope(ETaskTag::EGameThread); // Main thread is the game thread.

#if !(UE_BUILD_SHIPPING)

	// If "-waitforattach" or "-WaitForDebugger" was specified, halt startup and wait for a debugger to attach before continuing
	if (FParse::Param(CommandLine, TEXT("waitforattach")) || FParse::Param(CommandLine, TEXT("WaitForDebugger")))
	{
		while (!FPlatformMisc::IsDebuggerPresent());
		UE_DEBUG_BREAK();
	}

#endif

	// Override the stack size for the thread pool.
	FQueuedThreadPool::OverrideStackSize = 256 * 1024;

	// Initialize the engine.
	FString FinalCommandLine(CommandLine);
#if CRASH_REPORT_WITH_RECOVERY
	// -Messaging enables MessageBus transports required by Concert (Recovery Service).
	FinalCommandLine += TEXT(" -Messaging -EnablePlugins=\"UdpMessaging,ConcertSyncServer\"");
#endif
	GEngineLoop.PreInit(*FinalCommandLine);
	check(GConfig && GConfig->IsReadyForUse());

	// Increase the HttpSendTimeout to 5 minutes
	GConfig->SetFloat(TEXT("HTTP"), TEXT("HttpSendTimeout"), 5 * 60.0f, GEngineIni);

	// Make sure all UObject classes are registered and default properties have been initialized
	ProcessNewlyLoadedUObjects();

	// Tell the module manager is may now process newly-loaded UObjects when new C++ modules are loaded
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	// Load internal Concert plugins in the pre-default phase
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);

	// Load Concert Sync plugins in default phase
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default);

	// Initialize config.
	FCrashReportCoreConfig::Get();

	// Find the report to upload in the command line arguments
	ParseCommandLine(CommandLine);
	FPlatformErrorReport::Init();

	if (MonitorPid == 0) // Does not monitor any process.
	{
		if (AnalyticsEnabledFromCmd)
		{
			FCrashReportAnalytics::Initialize();
		}

		// Load error report generated by the process from disk
		FPlatformErrorReport ErrorReport = LoadErrorReport();

		// Apply project and crash overrides
		FString CrashSettingsIni;
		ErrorReport.FindFirstReportFileWithExtension(CrashSettingsIni, FGenericCrashContext::CrashConfigExtension);
		FCrashReportCoreConfig::Get().ApplyProjectOverrides(FPaths::Combine(ErrorReport.GetReportDirectory(), CrashSettingsIni));

		// At this point all overrides has been applied
		FCrashReportCoreConfig::Get().PrintSettingsToLog();
		
		SendErrorReport(ErrorReport, FApp::IsUnattended(), bImplicitSendFromCmd);

		if (AnalyticsEnabledFromCmd)
		{
			FCrashReportAnalytics::Shutdown();
		}
	}
	else // Launched in 'service mode - watches/serves a process'
	{
		FCrashReportAnalyticsSessionSummary::Get().Initialize(MonitorProcessGroupId, MonitorPid);

		if (!MonitorWritePipe)
		{
			FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("CRC/NoWritePipe"));
		}
		if (!MonitorReadPipe)
		{
			FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("CRC/NoReadPipe"));
		}

		const int32 IdealFramerate = 10;
		double PrevLoopStartTime = FPlatformTime::Seconds();
		const float IdealFrameTime = 1.0f / IdealFramerate;

		TSharedPtr<FRecoveryService> RecoveryServicePtr; // Note: Shared rather than Unique due to FRecoveryService only being a forward declaration in some builds

#if CRASH_REPORT_WITH_RECOVERY
		// Starts the disaster recovery service. This records transactions and allows users to recover from previous crashes.
		RecoveryServicePtr = MakeShared<FRecoveryService>(MonitorPid);
		FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Recovery/Started"));
#endif

		// Open the process with a restricted set of permissions (for security reasons).
		FProcHandle MonitoredProcess = OpenProcessForMonitoring(MonitorPid);

		// Loop until the monitored process dies.
		while (MonitoredProcess.IsValid() && FPlatformProcess::IsProcRunning(MonitoredProcess))
		{
			const double CurrLoopStartTime = FPlatformTime::Seconds();

			if (MonitorWritePipe && MonitorReadPipe)
			{
				// Check if the monitored process signaled a crash or an ensure, read the pipe data to avoid blocking the writer, but process the data only if CRC wasn't requested to exit.
				// This purposedly ignores any ensure that could be piped out just after a crash. (The way concurrent crash/ensures are handled/reported make this unlikely, but possible).
				FSharedCrashContext CrashContext;
				if (IsCrashReportAvailable(MonitorPid, CrashContext, MonitorReadPipe) && !IsEngineExitRequested())
				{
					FCrashReportAnalyticsSessionSummary::Get().OnCrashReportStarted(CrashContext.CrashType, CrashContext.ErrorMessage);

					const bool bReportCrashAnalyticInfo = CrashContext.UserSettings.bSendUsageData;
					if (bReportCrashAnalyticInfo)
					{
						FCrashReportAnalytics::Initialize(CrashContext.SessionContext.EpicAccountId);
					}

					FCrashReportAnalyticsSessionSummary::Get().OnCrashReportCollecting();

					// Build error report in memory.
					bool bCrashedThreadCallstackAvailable = false;
					FPlatformErrorReport ErrorReport = CollectErrorReport(RecoveryServicePtr.Get(), MonitorPid, CrashContext, MonitorWritePipe, bCrashedThreadCallstackAvailable);
					
					// Apply project and crash overrides
					FString CrashSettingsIni;
					ErrorReport.FindFirstReportFileWithExtension(CrashSettingsIni, FGenericCrashContext::CrashConfigExtension);
					FCrashReportCoreConfig::Get().ApplyProjectOverrides(FPaths::Combine(ErrorReport.GetReportDirectory(), CrashSettingsIni));

					// At this point all overrides has been applied
					FCrashReportCoreConfig::Get().PrintSettingsToLog();

					// Log cases where the PCallstack is missing. For analytics, this event hints that the remote app exited before CRC could walk the process stack and possibly means
					// that the timeout in the crashing process waiting for CRC to reply 'continue' is too short.
					if (!bCrashedThreadCallstackAvailable)
					{
						FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Report/NoPCallstack"));
					}

#if CRASH_REPORT_WITH_RECOVERY
					if (RecoveryServicePtr && !FPrimaryCrashProperties::Get()->bIsEnsure)
					{
						// Shutdown the recovery service. This will releases the recovery database file lock (not sharable) and let a new instance take it and offer the user to recover.
						FCrashReportAnalyticsSessionSummary::Get().LogEvent(TEXT("Recovery/Shutdown"));
						RecoveryServicePtr.Reset();
					}
#endif
					const bool bNoDialog = (CrashContext.UserSettings.bNoDialog || CrashContext.UserSettings.bImplicitSend) && CrashContext.UserSettings.bSendUnattendedBugReports;
					FCrashReportAnalyticsSessionSummary::Get().OnCrashReportProcessing(/*bIsUserInteractive*/!bNoDialog);
					const SubmitCrashReportResult Result = SendErrorReport(ErrorReport, bNoDialog, CrashContext.UserSettings.bImplicitSend);

					if (bReportCrashAnalyticInfo)
					{
						if (FCrashReportAnalytics::IsAvailable())
						{
							// If analytics is enabled make sure they are submitted now.
							FCrashReportAnalytics::GetProvider().BlockUntilFlushed(5.0f);
						}
						FCrashReportAnalytics::Shutdown();
					}

					FCrashReportAnalyticsSessionSummary::Get().OnCrashReportCompleted(Result != SubmitCrashReportResult::SuccessDiscarded && Result != SubmitCrashReportResult::Failed);
				}
			}

#if CRASH_REPORT_WITH_RECOVERY
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

			// Pump & Tick objects
			const double DeltaTime = CurrLoopStartTime - PrevLoopStartTime;
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			GFrameCounter++;
			UE::Stats::FStats::AdvanceFrame(false);

			// Run garbage collection for the UObjects for the rest of the frame or at least to 2 ms, but never more than 1 second.
			const float PurgeSeconds = IdealFrameTime - static_cast<float>(FPlatformTime::Seconds() - CurrLoopStartTime);
			IncrementalPurgeGarbage(true, FMath::Clamp(PurgeSeconds, 0.002f, 1.0f)));
#endif
			GLog->FlushThreadedLogs();
			
			// Throttle main thread fps by sleeping if we still have time.
			const float SleepSeconds = IdealFrameTime - static_cast<float>(FPlatformTime::Seconds() - CurrLoopStartTime);
			FPlatformProcess::Sleep(FMath::Clamp(SleepSeconds, 0.0f, 1.0f));
			PrevLoopStartTime = CurrLoopStartTime;
		}

		FCrashReportAnalyticsSessionSummary::Get().OnMonitoredAppDeath(MonitoredProcess);

#if CRASH_REPORT_WITH_MTBF
		{
			// Load the temporary crash context file.
			FSharedCrashContext TempCrashContext;
			FMemory::Memzero(TempCrashContext);
			if (LoadTempCrashContextFromFile(TempCrashContext, MonitorPid) && TempCrashContext.UserSettings.bSendUsageData)
			{
				FCrashReportAnalytics::Initialize(TempCrashContext.SessionContext.EpicAccountId);
				if (FCrashReportAnalytics::IsAvailable())
				{
					TOptional<int32> ExitCodeOpt;
					int32 ExitCode;
					if (FPlatformProcess::GetProcReturnCode(MonitoredProcess, &ExitCode))
					{
						ExitCodeOpt.Emplace(ExitCode);
					}

					auto HandleAbnormalShutdownFunc = [&TempCrashContext, &RecoveryServicePtr, &ExitCodeOpt]()
					{
						if (TempCrashContext.UserSettings.bSendUnattendedBugReports)
						{
							// Send a spoofed crash report in the case that we detect an abnormal shutdown has occurred
							HandleAbnormalShutdown(TempCrashContext, MonitorPid, MonitorWritePipe, RecoveryServicePtr, ExitCodeOpt);
						}
					};

					// Shutdown the session, sends the summary and if the session ended up abnormally (analyzing the summary), invoke the functor to spoof a crash report.
					FCrashReportAnalyticsSessionSummary::Get().Shutdown(&FCrashReportAnalytics::GetProvider(), HandleAbnormalShutdownFunc);
				}
				FCrashReportAnalytics::Shutdown();
			}
		}
#endif
		// Ensure to shutdown the summary analytics. If it was already shutdown above, this do nothing, otherwise, it destroys analytics data gathered by CRC.
		FCrashReportAnalyticsSessionSummary::Get().Shutdown();

		// Clean up the context file
		DeleteTempCrashContextFile(MonitorPid);

		// Clean up left-over context files that weren't cleaned up properly by previous instance(s) (because it was killed, it crashed or user logged out).
		DeleteExpiredTempCrashContext(FTimespan::FromDays(30));

		FPlatformProcess::CloseProc(MonitoredProcess);
	}

	FPrimaryCrashProperties::Shutdown();
	FPlatformErrorReport::ShutDown();

	RequestEngineExit(TEXT("CrashReportClientApp RequestExit"));

	// Allow the game thread to finish processing any latent tasks.
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	//FTaskGraphInterface::Shutdown();

	FEngineLoop::AppExit();
}
