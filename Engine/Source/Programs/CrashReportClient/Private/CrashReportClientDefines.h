// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef CRASH_REPORT_WITH_RECOVERY
#define CRASH_REPORT_WITH_RECOVERY 0
#endif

#ifndef CRASH_REPORT_WITH_MTBF
#define CRASH_REPORT_WITH_MTBF 0
#endif
