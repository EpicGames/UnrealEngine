// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportClientStyle.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Misc/Paths.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "CrashReportClientApp.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"

#if !CRASH_REPORT_UNATTENDED_ONLY

#define RootToContentDir ContentFromEngine

#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

TSharedPtr< FSlateStyleSet > FCrashReportClientStyle::StyleSet = nullptr;

void FCrashReportClientStyle::Initialize(const FCrashReportClientStyleOptions& Options)
{
	if(!StyleSet.IsValid())
	{
		StyleSet = Create(Options);
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
	}
}

void FCrashReportClientStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet);
	ensure(StyleSet.IsUnique());
	StyleSet.Reset();
}

namespace
{
	FString ContentFromEngine(const FString& RelativePath, const TCHAR* Extension)
	{
		static const FString ContentDir = FPaths::EngineDir() / TEXT("Content/Slate");
		return ContentDir / RelativePath + Extension;
	}
}

TSharedRef< FSlateStyleSet > FCrashReportClientStyle::Create(const FCrashReportClientStyleOptions& Options)
{
	TSharedRef<FSlateStyleSet> StyleRef = MakeShareable(new FSlateStyleSet("CrashReportClientStyle"));
	FSlateStyleSet& Style = StyleRef.Get();

	const FTextBlockStyle DefaultText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Bold", 10))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black);

	// Set the client app styles
	Style.Set(TEXT("Code"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FSlateColor(FLinearColor::White * 0.8f))
	);

	Style.Set(TEXT("Title"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Bold", 12))
	);

	Style.Set(TEXT("Status"), FTextBlockStyle(DefaultText)
		.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
	);

	const FVector2D Icon16x16( 16.0f, 16.0f );
	FSlateBrush* GenericWhiteBox = new IMAGE_BRUSH( "Old/White", Icon16x16 );

	// Load the screenshot image, if one is specified.
	const FString& ScreenshotPath = Options.OptionalCrashScreenshot.Filepath;
	bool bHasValidScreenshotFile = FPaths::FileExists(ScreenshotPath);
	if (bHasValidScreenshotFile && !Options.OptionalCrashScreenshot.ImageSize.IsZero())
	{
		Style.Set("CrashScreenshot", new FSlateImageBrush(ScreenshotPath, Options.OptionalCrashScreenshot.ImageSize));
	}

	// Scrollbar
	const FScrollBarStyle ScrollBar = FScrollBarStyle()
		.SetVerticalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetVerticalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetHorizontalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetHorizontalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetNormalThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetDraggedThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetHoveredThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)));

	// SEditableTextBox defaults...
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FEditableTextBoxStyle NormalEditableTextBoxStyle = FEditableTextBoxStyle()
		.SetTextStyle(NormalText)
		.SetBackgroundImageNormal(*GenericWhiteBox)
		.SetBackgroundImageHovered(*GenericWhiteBox)
		.SetBackgroundImageFocused(*GenericWhiteBox)
		.SetBackgroundImageReadOnly(*GenericWhiteBox)
		.SetScrollBarStyle(ScrollBar);
	{
		Style.Set( "NormalEditableTextBox", NormalEditableTextBoxStyle );
	}

	// RichText
	const FTextBlockStyle CrashReportDataStyle = FTextBlockStyle()
		.SetFont( DEFAULT_FONT( "Italic", 9 ) )
		.SetColorAndOpacity( FSlateColor( FLinearColor::White * 0.5f ) )
		.SetShadowOffset( FVector2D::ZeroVector )
		.SetShadowColorAndOpacity( FLinearColor::Black );

	Style.Set( "CrashReportDataStyle", CrashReportDataStyle );

	FButtonStyle DarkHyperlinkButton = FButtonStyle()
		.SetNormal( BORDER_BRUSH( "Old/HyperlinkDotted", FMargin( 0, 0, 0, 3 / 16.0f ), FSlateColor( FLinearColor::White * 0.5f ) ) )
		.SetPressed( FSlateNoResource() )
		.SetHovered( BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin( 0, 0, 0, 3 / 16.0f ), FSlateColor( FLinearColor::White * 0.5f ) ) );

	const FHyperlinkStyle DarkHyperlink = FHyperlinkStyle()
		.SetUnderlineStyle( DarkHyperlinkButton )
		.SetTextStyle( CrashReportDataStyle )
		.SetPadding( FMargin( 0.0f ) );

	Style.Set( "RichText.Hyperlink", DarkHyperlink );

	Style.Set("ToolPanel.GroupBorder", new FSlateColorBrush(FStyleColors::Panel));

	return StyleRef;
}

const ISlateStyle& FCrashReportClientStyle::Get()
{
	return *StyleSet;
}

#undef RootToContentDir

#endif // !CRASH_REPORT_UNATTENDED_ONLY
