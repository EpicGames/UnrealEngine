// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CrashReportClientApp.h"

#if !CRASH_REPORT_UNATTENDED_ONLY

#include "Styling/SlateStyle.h"

struct FCrashReportClientStyleOptions {
	struct {
		FString Filepath;
		FVector2D ImageSize = { 0,0 };
	} OptionalCrashScreenshot;
};

/** Slate styles for the crash report client app */
class FCrashReportClientStyle
{
public:
	/**
	 * Set up specific styles for the crash report client app
	 */
	static void Initialize(const FCrashReportClientStyleOptions& Options);

	/**
	 * Tidy up on shut-down
	 */
	static void Shutdown();

	/*
	 * Access to singleton style object
	 */ 
	static const ISlateStyle& Get();

private:
	static TSharedRef<FSlateStyleSet> Create(const FCrashReportClientStyleOptions& Options);

	/** Singleton style object */
	static TSharedPtr<FSlateStyleSet> StyleSet;
};

#endif // !CRASH_REPORT_UNATTENDED_ONLY
