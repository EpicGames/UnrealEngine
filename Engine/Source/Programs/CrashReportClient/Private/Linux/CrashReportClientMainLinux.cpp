// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/FeedbackContext.h"
#include "CrashReportClientApp.h"
#include "Unix/UnixPlatformCrashContext.h"
#include <locale.h>

extern int32 ReportCrash(const FUnixCrashContext& Context);	// FIXME: handle expose it someplace else?

/**
 * Because crash reporters can crash, too
 */
void CrashReporterCrashHandler(const FGenericCrashContext& GenericContext)
{
	// at this point we should already be using malloc crash handler (see PlatformCrashHandler)

	const FUnixCrashContext& Context = static_cast< const FUnixCrashContext& >( GenericContext );

	printf("CrashHandler: Signal=%d\n", Context.Signal);
	const_cast< FUnixCrashContext& >(Context).CaptureStackTrace(Context.FirstCrashHandlerFrame);
	if (GLog)
	{
		GLog->Panic();
	}
	if (GWarn)
	{
		GWarn->Flush();
	}
	if (GError)
	{
		GError->Flush();
		GError->HandleError();
	}
	FPlatformMisc::RequestExit(true);
}


/**
 * main(), called when the application is started
 */
int main(int argc, const char *argv[])
{
	FPlatformMisc::SetGracefulTerminationHandler();
	FPlatformMisc::SetCrashHandler(CrashReporterCrashHandler);

	FString SavedCommandLine;

	for (int32 Option = 1; Option < argc; Option++)
	{
		SavedCommandLine += TEXT(" ");
		SavedCommandLine += UTF8_TO_TCHAR(argv[Option]);	// note: technically it depends on locale
	}

	// assume unattended if we don't have X11 display
	if (getenv("DISPLAY") == nullptr)
	{
		SavedCommandLine += TEXT(" -unattended");
	}

	// Run the app
	RunCrashReportClient(*SavedCommandLine);

	return 0;
}
