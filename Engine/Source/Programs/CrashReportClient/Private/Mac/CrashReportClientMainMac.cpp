// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportClientApp.h"

#include "Apple/ScopeAutoreleasePool.h"
#include "HAL/ExceptionHandling.h"
#include "Mac/MacPlatformCrashContext.h"
#include "Mac/MacPlatformProcess.h"
#include "Mac/CocoaThread.h"
#include "HAL/PlatformApplicationMisc.h"

/**
 * Because crash reporters can crash, too - only used for Sandboxed applications
 */
void CrashReporterCrashHandler(const FGenericCrashContext& GenericContext)
{
	// We never emit a crash report in Sandboxed builds from CRC as if we do, then the crashed application's
	// crash report is overwritten by the CRC's when trampolining into the Apple Crash Reporter.
	_Exit(0);
}

static FString GSavedCommandLine;
static bool GIsUnattended = false;

@interface UEAppDelegate : NSObject <NSApplicationDelegate, NSFileManagerDelegate>
{
}

@end

@implementation UEAppDelegate

//handler for the quit apple event used by the Dock menu
- (void)handleQuitEvent:(NSAppleEventDescriptor*)Event withReplyEvent:(NSAppleEventDescriptor*)ReplyEvent
{
    [self requestQuit:self];
}

- (IBAction)requestQuit:(id)Sender
{
	RequestEngineExit(TEXT("Mac CrashReportClient RequestQuit"));
}

- (void) runGameThread:(id)Arg
{
	FPlatformMisc::SetGracefulTerminationHandler();
	// For sandboxed applications CRC can never report a crash, or we break trampolining into Apple's crash reporter.
	if(FPlatformProcess::IsSandboxedApplication())
	{
		FPlatformMisc::SetCrashHandler(CrashReporterCrashHandler);
	}
	
	RunCrashReportClient(*GSavedCommandLine);
    
    dispatch_async(dispatch_get_main_queue(), ^{
        [NSApp terminate: nil];
    });
}

- (NSApplicationTerminateReply)applicationShouldTerminate:(NSApplication *)Sender;
{
	if(!IsEngineExitRequested() || ([NSThread gameThread] && [NSThread gameThread] != [NSThread mainThread]))
	{
		[self requestQuit:self];
		return NSTerminateLater;
	}
	else
	{
		return NSTerminateNow;
	}
}

- (void)applicationDidFinishLaunching:(NSNotification *)Notification
{
	//install the custom quit event handler
    NSAppleEventManager* appleEventManager = [NSAppleEventManager sharedAppleEventManager];
    [appleEventManager setEventHandler:self andSelector:@selector(handleQuitEvent:withReplyEvent:) forEventClass:kCoreEventClass andEventID:kAEQuitApplication];
	
	if (!GIsUnattended)
	{
		FPlatformApplicationMisc::ActivateApplication();
	}
	RunGameThread(self, @selector(runGameThread:));
}

@end

int main(int argc, char *argv[])
{
	for (int32 Option = 1; Option < argc; Option++)
	{
		GSavedCommandLine += TEXT(" ");
		FString Argument(ANSI_TO_TCHAR(argv[Option]));
		if (Argument.Contains(TEXT(" ")))
		{
			if (Argument.Contains(TEXT("=")))
			{
				FString ArgName;
				FString ArgValue;
				Argument.Split( TEXT("="), &ArgName, &ArgValue );
				Argument = FString::Printf( TEXT("%s=\"%s\""), *ArgName, *ArgValue );
			}
			else
			{
				Argument = FString::Printf(TEXT("\"%s\""), *Argument);
			}
		}
		else if (Argument.ToLower() == TEXT("-unattended"))
		{
			GIsUnattended = true;
		}
		GSavedCommandLine += Argument;
	}
	
	SCOPED_AUTORELEASE_POOL;
	[NSApplication sharedApplication];
	[NSApp setDelegate:[UEAppDelegate new]];
	[NSApp run];
	return 0;
}
