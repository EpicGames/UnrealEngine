// Copyright Epic Games, Inc. All Rights Reserved.

#include "MainLoopTiming.h"
#include "HAL/PlatformProcess.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Ticker.h"
#include "CrashReportClientApp.h"

#include "Async/TaskGraphInterfaces.h"

#if !CRASH_REPORT_UNATTENDED_ONLY
	#include "Framework/Application/SlateApplication.h"
#endif

FMainLoopTiming::FMainLoopTiming(float InIdealTickRate, EMainLoopOptions::Type Options)
	: IdealFrameTime(1.f / InIdealTickRate)
	, bTickSlate(Options & EMainLoopOptions::UsingSlate)
{
}

void FMainLoopTiming::Tick()
{
	static double ActualDeltaTime = IdealFrameTime;
	static double LastTime = FPlatformTime::Seconds();

	// Tick app logic
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
	FTSTicker::GetCoreTicker().Tick(ActualDeltaTime);

#if !CRASH_REPORT_UNATTENDED_ONLY
	// Tick SlateApplication
	if (bTickSlate)
	{
		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();
	}
#endif // !CRASH_REPORT_UNATTENDED_ONLY

	// Sleep Throttling
	// Copied from Community Portal - should be shared
	FPlatformProcess::Sleep(FMath::Max<float>(0, IdealFrameTime - (FPlatformTime::Seconds() - LastTime)));

	// Calculate deltas
	const double AppTime = FPlatformTime::Seconds();
	ActualDeltaTime = AppTime - LastTime;
	LastTime = AppTime;	
}
