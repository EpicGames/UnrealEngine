// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Options governing what the main loop object should Tick */
namespace EMainLoopOptions
{
	enum Type
	{
		/** No flags set: only update the core ticker */
		CoreTickerOnly = 0,

		/** Call Tick on Slate */
		UsingSlate = 0x01,

		/** Default to running Slate if no options passed */
		Default = UsingSlate
	};
}

/**
 * Calls Tick on Slate and the core ticker at a set rate
 */
class FMainLoopTiming
{
public:
	/**
	 * Constructor: set up initial timing values
	 */
	explicit FMainLoopTiming(float IdealTickRate, EMainLoopOptions::Type Options = EMainLoopOptions::Default);

	/**
	 * Tick core application objects and throttle rate as requested in constructor
	 */
	void Tick();

private:
	/** Interval between each Tick that we're aiming for */
	float IdealFrameTime;

	/** Should Slate's Tick be called? */
	bool bTickSlate;
};
