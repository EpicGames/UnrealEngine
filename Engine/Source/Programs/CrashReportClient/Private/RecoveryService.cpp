// Copyright Epic Games, Inc. All Rights Reserved.

#include "RecoveryService.h"

#if CRASH_REPORT_WITH_RECOVERY

#include "HAL/FileManager.h"
#include "CrashReportClient.h" // For CrashReportClientLog
#include "Interfaces/IPluginManager.h"
#include "IMessagingModule.h"
#include "ConcertSettings.h"
#include "ConcertSyncSessionFlags.h"
#include "IConcertServer.h"
#include "IConcertSession.h"
#include "IConcertSyncServer.h"
#include "IConcertSyncServerModule.h"
#include "ConcertMessageData.h"
#include "Runtime/Launch/Resources/Version.h"
#include "ConcertLocalFileSharingService.h"

static const TCHAR RecoveryServiceName[] = TEXT("Disaster Recovery Service");

bool FRecoveryService::CollectFiles(const FString& DestDir, bool bMetaDataOnly, bool bAnonymizeMetaData)
{
	auto LogError = [](const TCHAR* Reason)
	{
		UE_LOGF(CrashReportClientLog, Error, "Failed to collect recovery session file(s). %ls", Reason);
	};

	if (!Server)
	{
		LogError(TEXT("The recovery service is not running."));
		return false;
	}
	else if (!IFileManager::Get().DirectoryExists(*DestDir))
	{
		LogError(TEXT("The destination folder doesn't exist."));
		return false;
	}

	FGuid ExportedSessionId = GetRecoverySessionId();
	if (!ExportedSessionId.IsValid())
	{
		LogError(TEXT("The session session could not be found."));
		return false;
	}

	FText ErrorMsg;
	FConcertSessionFilter Filter;
	Filter.bMetaDataOnly = bMetaDataOnly;
	if (!Server->GetConcertServer()->ExportSession(ExportedSessionId, Filter, DestDir, bAnonymizeMetaData, ErrorMsg))
	{
		LogError(TEXT("Server failed to export the session."));
		return false;
	}

	return true;
}

bool FRecoveryService::Startup()
{
#if UE_BUILD_SHIPPING && (!defined(PLATFORM_SUPPORTS_MESSAGEBUS) || !PLATFORM_SUPPORTS_MESSAGEBUS)
	#error PLATFORM_SUPPORTS_MESSAGEBUS was explicitly defined in CrashReportClient.Target.cs for shipping configuration. MessageBus is required by Concert. Ensure it is still enabled.
#endif
	if (!IMessagingModule::Get().GetDefaultBus())
	{
		UE_LOGF(CrashReportClientLog, Error, "MessageBus is not enabled in this configuration. Recovery service will be disabled!");
		return false;
	}

	if (!IConcertSyncServerModule::IsAvailable())
	{
		UE_LOGF(CrashReportClientLog, Error, "ConcertSyncServer Module is missing. Recovery service will be disabled!");
		return false;
	}

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("UdpMessaging"));
	if (!Plugin || !Plugin->IsEnabled())
	{
		// The UdpMessaging plugin should be added to the {appname}.Target.cs build file.
		UE_LOGF(CrashReportClientLog, Error, "The 'UDP Messaging' plugin is disabled. The Concert server only supports UDP protocol. Recovery service will be disabled!");
		return false;
	}

	// Setup the disaster recovery server configuration
	UConcertServerConfig* ServerConfig = IConcertSyncServerModule::Get().ParseServerSettings(FCommandLine::Get());
	ServerConfig->bAutoArchiveOnReboot = false; // Skip archiving, disaster recovery restore a live session by copying it, this saves the step of archiving.
	ServerConfig->bAutoArchiveOnShutdown = false; // Skip archiving, this can takes several minutes. It is more efficient to let the session 'live' and 'copy' it when restoring.
	ServerConfig->EndpointSettings.RemoteEndpointTimeoutSeconds = 0; // Ensure the endpoints never time out (and are kept alive automatically by Concert).
	ServerConfig->bMountDefaultSessionRepository = false; // Let the client mount its own repository to support concurrent recovery server and prevent them from concurrently accessing non-sharable database files.
	ServerConfig->AuthorizedClientKeys.Add(ServerConfig->ServerName); // The disaster recovery client is configured to use the unique server name as key to identify itself.

	FConcertSessionFilter AutoArchiveSessionFilter;
	AutoArchiveSessionFilter.bIncludeIgnoredActivities = true;

	// Start disaster recovery server.
	Server = IConcertSyncServerModule::Get().CreateServer(TEXT("DisasterRecovery"), AutoArchiveSessionFilter);
	Server->SetFileSharingService(MakeShared<FConcertLocalFileSharingService>(Server->GetConcertServer()->GetRole()));
	Server->Startup(ServerConfig, EConcertSyncSessionFlags::Default_DisasterRecoverySession);

	UE_LOGF(CrashReportClientLog, Display, "%ls Initialized (Name: %ls, Version: %d.%d, Role: %ls)", RecoveryServiceName, *Server->GetConcertServer()->GetServerInfo().ServerName, ENGINE_MAJOR_VERSION, ENGINE_MINOR_VERSION, *Server->GetConcertServer()->GetRole());
	return true;
}

void FRecoveryService::Shutdown()
{
	if (Server)
	{
		Server->Shutdown();
		Server.Reset();
		UE_LOGF(CrashReportClientLog, Display, "%ls Shutdown", RecoveryServiceName);
	}
}

FGuid FRecoveryService::GetRecoverySessionId() const
{
	FGuid SessionId;
	int32 SessionSeqNum = -1;

	// As long as the Concert server is up, the session would remain live (it's going to be archived when the server shutdown or reboot).
	for (TSharedPtr<IConcertServerSession>& Session : Server->GetConcertServer()->GetSessions())
	{
		// As convention, the disaster recovery session names starts with the server name, followed by a sequence number, the project name and date time. (See RecoveryService::MakeSessionName())
		if (Session->GetName().StartsWith(Server->GetConcertServer()->GetServerInfo().ServerName))
		{
			// The user may have enabled/disabled the recovery service few times and as result, several live sessions will be available. Need to pick the last one. The highest sequence number
			// in the session name corresponds to the last session created.
			int32 SeqNum = 0;
			RecoveryService::TokenizeSessionName(Session->GetName(), nullptr, &SeqNum, nullptr, nullptr);
			if (SeqNum > SessionSeqNum)
			{
				SessionId = Session->GetId();
			}
		}
	}

	return SessionId; // Uninitialized Guid (invalid) means not found.
}

#endif
