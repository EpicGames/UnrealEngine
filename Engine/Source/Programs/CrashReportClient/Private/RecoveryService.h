// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CrashReportClientDefines.h"

#if CRASH_REPORT_WITH_RECOVERY

class IConcertSyncServer;

/** Manage the life time of the Disaster recovery service. */
class FRecoveryService
{
public:
	FRecoveryService(uint64 InClientPid) : MonitorPid(InClientPid) { Startup(); }
	~FRecoveryService() { Shutdown(); }

	/** Collects the session files. */
	bool CollectFiles(const FString& DestDir, bool bMetaDataOnly = true, bool bAnonymizeMetaData = false);

private:
	bool Startup();
	void Shutdown();

	FString GetRecoveryServiceWorkingDir() const;
	FString GetRecoveryServiceArchivedDir() const;
	FGuid GetRecoverySessionId() const;

private:
	TSharedPtr<IConcertSyncServer> Server;
	uint64 MonitorPid = 0;
};

#endif
