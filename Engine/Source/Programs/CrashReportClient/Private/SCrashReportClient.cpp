// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCrashReportClient.h"

#if !CRASH_REPORT_UNATTENDED_ONLY

#include "CrashReportClientStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Images/SThrobber.h"
#include "CrashDescription.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/EngineBuildSettings.h"

#define LOCTEXT_NAMESPACE "CrashReportClient"

static void OnBrowserLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* UrlPtr = Metadata.Find(TEXT("href"));
	if(UrlPtr)
	{
		FPlatformProcess::LaunchURL(**UrlPtr, nullptr, nullptr);
	}
}

static void OnViewCrashDirectory( const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* UrlPtr = Metadata.Find( TEXT( "href" ) );
	if (UrlPtr)
	{
		FPlatformProcess::ExploreFolder( **UrlPtr );
	}
}

void SCrashReportClient::Construct(const FArguments& InArgs, const TSharedRef<FCrashReportClient>& Client, bool bSimpleDialog)
{
	CrashReportClient = Client;
	bHasUserCommentErrors = false;
	bIsUserCommentEmpty = true;
	bHideSubmitAndRestart = InArgs._bHideSubmitAndRestart;

	// Fallback if the runtime has not set detailed message
	FText BugSubmissionUrl = FText::FromString(TEXT("https://unrealbugsubmissions.epicgames.com/"));
	FText CrashDetailedMessage = LOCGEN_FORMAT_ORDERED(
		LOCTEXT("CrashDetailed", "We are very sorry that this crash occurred. Our goal is to prevent crashes like this from occurring in the future. Please help us track down and fix this crash by providing detailed information about what you were doing so that we may reproduce the crash and fix it quickly. You can also log a Bug Report with us using the <a id=\"browser\" href=\"{0}\" style=\"Hyperlink\">Bug Submission Form</> and work directly with support staff to report this issue.\n\nThanks for your help in improving the Unreal Engine."),
		BugSubmissionUrl
	);
	if (FPrimaryCrashProperties::Get()->IsValid())
	{
		FString CrashDetailedMessageString = FPrimaryCrashProperties::Get()->CrashReporterMessage.AsString();
		if (!CrashDetailedMessageString.IsEmpty())
		{
			CrashDetailedMessage = FText::FromString(CrashDetailedMessageString);
		}
	}

	if (bSimpleDialog)
	{
		ConstructSimpleDialog(Client, CrashDetailedMessage);
	}
	else
	{ 
		ConstructDetailedDialog(Client, CrashDetailedMessage);
	}

	FSlateApplication::Get().SetUnhandledKeyDownEventHandler(FOnKeyEvent::CreateSP(this, &SCrashReportClient::OnUnhandledKeyDown));
}

void SCrashReportClient::ConstructDetailedDialog(const TSharedRef<FCrashReportClient>& Client, const FText& CrashDetailedMessage)
{
	auto CrashedAppName = FPrimaryCrashProperties::Get()->IsValid() ? FPrimaryCrashProperties::Get()->GameName : TEXT("");

	// Set the text displaying the name of the crashed app, if available
	const FText CrashedAppText = CrashedAppName.IsEmpty() ?
		LOCTEXT( "CrashedAppNotFound", "An unknown process has crashed" ) :
		LOCTEXT( "CrashedAppUnreal", "An Unreal process has crashed: " );

	const FText CrashReportDataText = FText::Format( 
		LOCTEXT( "CrashReportData", "Crash reports comprise diagnostics files (<a id=\"browser\" href=\"{0}\" style=\"Richtext.Hyperlink\">click here to view directory</>) and the following summary information: " ),
		FText::FromString( CrashReportClient->GetCrashDirectory()) );

	const FText IncludeScreenshotText = LOCTEXT("CrashReportIncludeScreenshot", "Include screenshot in the crash report");

	const FSlateBrush* Screenshot = FCrashReportClientStyle::Get().GetOptionalBrush("CrashScreenshot", nullptr, nullptr);

	SVerticalBox::FSlot::FSlotArguments ScreenshotSlot = SVerticalBox::Slot();
	if (Screenshot)
	{
		ScreenshotSlot.Padding(FMargin(4, 10))
		.MaxHeight(500)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4)
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SCheckBox)
									.IsChecked_Lambda([this]()
										{
											return bIncludeScreenshotInCrashReport ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
										})
									.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
										{
											bIncludeScreenshotInCrashReport = InCheckBoxState == ECheckBoxState::Checked;
										})
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(4, 1,1,1)
							[
								SNew(SRichTextBlock)
									.Text(IncludeScreenshotText)
							]
					]
					+ SVerticalBox::Slot()
					[
						SNew(SScaleBox)
							.Stretch(EStretch::ScaleToFit)
							[
								SNew(SImage)
									.IsEnabled(Screenshot != nullptr)
									.Image(Screenshot)
									.ColorAndOpacity_Lambda([this]() {
										return FLinearColor(1.0f, 1.0f, 1.0f, bIncludeScreenshotInCrashReport ? 1.0f : 0.4f);
									})
							]
					]
			];
	}
	else
	{
		ScreenshotSlot.AutoHeight();
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCrashReportClientStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			// Stuff anchored to the top
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FCrashReportClientStyle::Get(), "Title")
					.Text(CrashedAppText)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FCrashReportClientStyle::Get(), "Title")
					.Text(FText::FromString(CrashedAppName))
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 4, 10 ) )
			[
				SNew( SRichTextBlock )
				.Text(CrashDetailedMessage)
				.AutoWrapText(true)
				+ SRichTextBlock::HyperlinkDecorator( TEXT("browser"), FSlateHyperlinkRun::FOnClick::CreateStatic( &OnBrowserLinkClicked ) )
			]

			+ScreenshotSlot

			+SVerticalBox::Slot()
			.Padding(FMargin(4, 10, 4, 4))
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+SSplitter::Slot()
				.Value(0.3f)
				[
					SNew( SOverlay )
					+ SOverlay::Slot()
					[
						SAssignNew( CrashDetailsInformation, SMultiLineEditableTextBox )
						.Style( &FCrashReportClientStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>( "NormalEditableTextBox" ) )
						.OnTextCommitted( CrashReportClient.ToSharedRef(), &FCrashReportClient::UserCommentChanged )
						.OnTextChanged( this, &SCrashReportClient::OnUserCommentTextChanged)
						.Font( FCoreStyle::GetDefaultFontStyle("Regular", 9) )
						.AutoWrapText( true )
						.BackgroundColor( FSlateColor( FLinearColor::Black ) )
						.ForegroundColor( FSlateColor( FLinearColor::White * 0.8f ) )
					]

					// HintText is not implemented in SMultiLineEditableTextBox, so this is a workaround.
					+ SOverlay::Slot()
					[
						SNew(STextBlock)
						.Margin( FMargin(4,2,0,0) )
						.Font( FCoreStyle::GetDefaultFontStyle("Italic", 9) )
						.ColorAndOpacity( FSlateColor( FLinearColor::White * 0.5f ) )
						.Text( LOCTEXT( "CrashProvide", "Please provide detailed information about what you were doing when the crash occurred." ) )
						.Visibility( this, &SCrashReportClient::IsHintTextVisible )
					]	
				]

				+SSplitter::Slot()
				.Value(0.7f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SOverlay)

						+ SOverlay::Slot()			
						[
							SNew(SColorBlock)
							.Color(FLinearColor::Black)
						]

						+ SOverlay::Slot()
						[
							SNew( SRichTextBlock )
							.Margin( FMargin( 4, 2, 0, 8 ) )
							.TextStyle( &FCrashReportClientStyle::Get().GetWidgetStyle<FTextBlockStyle>( "CrashReportDataStyle" ) )
							.Text( CrashReportDataText )
							.AutoWrapText( true )
							.DecoratorStyleSet( &FCrashReportClientStyle::Get() )
							+ SRichTextBlock::HyperlinkDecorator( TEXT( "browser" ), FSlateHyperlinkRun::FOnClick::CreateStatic( &OnViewCrashDirectory ) )
						]
					]

					+ SVerticalBox::Slot()
					.FillHeight(0.7f)
					[
						SNew(SOverlay)
		
						+ SOverlay::Slot()
						[
							SNew( SMultiLineEditableTextBox )
							.Style( &FCrashReportClientStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>( "NormalEditableTextBox" ) )
							.Font( FCoreStyle::GetDefaultFontStyle("Regular", 8) )
							.AutoWrapText( false )
							.IsReadOnly( true )
							.ReadOnlyForegroundColor( FSlateColor( FLinearColor::White * 0.8f) )
							.BackgroundColor( FSlateColor( FLinearColor::Black ) )
							.ForegroundColor( FSlateColor( FLinearColor::White * 0.8f ) )
							.Text( Client, &FCrashReportClient::GetDiagnosticText )
						]


						+ SOverlay::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SThrobber)
							.Visibility(CrashReportClient.ToSharedRef(), &FCrashReportClient::IsThrobberVisible)
							.NumPieces(5)
						]
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 4, 12, 4, 4 ) )
			[
				SNew( SHorizontalBox )
				.Visibility( FCrashReportCoreConfig::Get().GetHideLogFilesOption() ? EVisibility::Collapsed : EVisibility::Visible )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign( VAlign_Center )
				[
					SNew( SCheckBox )
					.IsChecked( FCrashReportCoreConfig::Get().GetSendLogFile() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked )
					.OnCheckStateChanged( CrashReportClient.ToSharedRef(), &FCrashReportClient::SendLogFile_OnCheckStateChanged )
				]

				+ SHorizontalBox::Slot()
				.FillWidth( 1.0f )
				.VAlign( VAlign_Center )
				[
					SNew( STextBlock )
					.AutoWrapText( true )
					.Text( LOCTEXT( "IncludeLogs", "Include log files with submission. I understand that logs contain some personal information such as my system and user name." ) )
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 4, 4 ) )
			[
				SNew(SHorizontalBox)
				.Visibility( FCrashReportCoreConfig::Get().GetHideAllowToBeContactedOption() ? EVisibility::Collapsed : EVisibility::Visible )

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked( FCrashReportCoreConfig::Get().GetAllowToBeContacted() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked )
					.IsEnabled( !FEngineBuildSettings::IsInternalBuild() )
					.OnCheckStateChanged(CrashReportClient.ToSharedRef(), &FCrashReportClient::AllowToBeContacted_OnCheckStateChanged)
				]

				
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.IsEnabled( !FEngineBuildSettings::IsInternalBuild() )
					.Text_Static(&SCrashReportClient::GetContactText)
				]
			]

			// Stuff anchored to the bottom
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin(4, 4+16, 4, 4) )
			[
				SNew(SHorizontalBox)
	
				+ SHorizontalBox::Slot()
				.HAlign( HAlign_Center )
				.VAlign( VAlign_Center )
				.AutoWidth()
				.Padding( FMargin( 0 ) )
				[
					SNew( SButton )
					.ContentPadding( FMargin( 8, 2 ) )
					.Text( LOCTEXT( "CloseWithoutSending", "Close Without Sending" ) )
					.OnClicked( Client, &FCrashReportClient::CloseWithoutSending )
					.Visibility(FCrashReportCoreConfig::Get().IsAllowedToCloseWithoutSending() ? EVisibility::Visible : EVisibility::Hidden)
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0)
				[			
					SNew(SSpacer)
				]

#if PLATFORM_WINDOWS
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding( FMargin(6) )
				[
					SNew(SButton)
					.ContentPadding( FMargin(8,2) )
					.Text(LOCTEXT("CopyFiles", "Copy Files To Clipboard"))
					.OnClicked(Client, &FCrashReportClient::CopyFilesToClipboard)
					.Visibility(FCrashReportCoreConfig::Get().IsAllowedToCopyFilesToClipboard() ? EVisibility::Visible : EVisibility::Hidden)
				]
#endif

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding( FMargin(6) )
				[
					SNew(SButton)
					.ContentPadding( FMargin(8,2) )
					.Text(LOCTEXT("Send", "Send and Close"))
					.OnClicked(Client, &FCrashReportClient::Submit, bIncludeScreenshotInCrashReport)
					.IsEnabled(this, &SCrashReportClient::IsSendEnabled)
					.ToolTipText_Static(&SCrashReportClient::GetSendTooltip)
				]

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding( FMargin(0) )
				[
					SNew(SButton)
					.ContentPadding( FMargin(8,2) )
					.Text(LOCTEXT("SendAndRestartEditor", "Send and Restart"))
					.OnClicked(Client, &FCrashReportClient::SubmitAndRestart)
					.IsEnabled(this, &SCrashReportClient::IsSendEnabled)
					.Visibility( bHideSubmitAndRestart || FCrashReportCoreConfig::Get().GetHideRestartOption() ? EVisibility::Collapsed : EVisibility::Visible )
					.ToolTipText_Static(&SCrashReportClient::GetSendTooltip)
				]
			]
		]
	];
}

void SCrashReportClient::ConstructSimpleDialog(const TSharedRef<FCrashReportClient>& Client, const FText& CrashDetailedMessage)
{
	FString CrashedAppName = FPrimaryCrashProperties::Get()->IsValid() ? FPrimaryCrashProperties::Get()->GameName : TEXT("");
	// GameNames have taken on a number of prefixes over the years. Try to strip them all off.
	if (!CrashedAppName.RemoveFromStart(TEXT("UE4-")))
	{
		if (!CrashedAppName.RemoveFromStart(TEXT("UE5-")))
		{
			CrashedAppName.RemoveFromStart(TEXT("UE-"));
		}
	}
	CrashedAppName.RemoveFromEnd(TEXT("Game"));

	// This simple dialog is used for the unattended mode where the base crash report is sent
	// unconditionally, but we still show options for optional attachments such as the screenshot.
	const FText SendScreenshotText = LOCTEXT("CrashReportSendScreenshot", "Send screenshot");
	const FText ScreenshotDescriptionText = LOCTEXT("CrashReportScreenshotDescription",
		"The following screenshot was captured when the application crashed. You can optionally include this screenshot as part of the crash report.");
	const FSlateBrush* Screenshot = FCrashReportClientStyle::Get().GetOptionalBrush("CrashScreenshot", nullptr, nullptr);

	SVerticalBox::FSlot::FSlotArguments ScreenshotDescriptionSlot = SVerticalBox::Slot();
	SVerticalBox::FSlot::FSlotArguments ScreenshotSlot = SVerticalBox::Slot();
	if (Screenshot)
	{
		ScreenshotDescriptionSlot
		.AutoHeight()
		.Padding(FMargin(4, 10))
		[
			SNew(SRichTextBlock)
			.Text(ScreenshotDescriptionText)
			.AutoWrapText(true)
		];

		ScreenshotSlot
		.Padding(FMargin(4, 4))
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SImage)
				.Image(Screenshot)
			]
		];
	}
	else
	{
		ScreenshotDescriptionSlot.AutoHeight();
		ScreenshotSlot.AutoHeight();
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCrashReportClientStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			// Stuff anchored to the top
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FCrashReportClientStyle::Get(), "Title")
					.Text(FText::FromString(CrashedAppName))
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 4, 10 ) )
			[
				SNew( SRichTextBlock )
				.Text(CrashDetailedMessage)
				.AutoWrapText(true)
				+ SRichTextBlock::HyperlinkDecorator( TEXT("browser"), FSlateHyperlinkRun::FOnClick::CreateStatic( &OnBrowserLinkClicked ) )
			]

			+ScreenshotDescriptionSlot
			+ScreenshotSlot

			// Stuff anchored to the bottom
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin(4, 4, 4, 4) )
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				.Padding(0)
				[
					SNew(SSpacer)
				]

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				.Padding( FMargin(6) )
				[
					SNew(SButton)
					.ContentPadding(FMargin(8, 2))
					.Text(SendScreenshotText)
					.Visibility(Screenshot ? EVisibility::Visible : EVisibility::Collapsed)
					.OnClicked_Lambda([Client]() {
						return Client->SubmitOptionalAttachmentsAndClose();
					 })
				]

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				.AutoWidth()
				.Padding( FMargin(6) )
				[
					SNew(SButton)
					.ContentPadding( FMargin(8,2) )
					.Text(Screenshot ? LOCTEXT("CloseWithoutSending", "Close Without Sending") : LOCTEXT("Close", "Close"))
					.OnClicked(Client, &FCrashReportClient::Close)
				]		
			]
		]
	];
}


FReply SCrashReportClient::OnUnhandledKeyDown(const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();
	if (Key == EKeys::Enter)
	{
		CrashReportClient->Submit();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SCrashReportClient::OnUserCommentTextChanged(const FText& NewText)
{
	FText ErrorMessage = FText::GetEmpty();
	bHasUserCommentErrors = false;
	bIsUserCommentEmpty = NewText.IsEmpty();

	int SizeLimit = FCrashReportCoreConfig::Get().GetUserCommentSizeLimit();
	int Size = NewText.ToString().Len();
	if (Size > SizeLimit)
	{
		bHasUserCommentErrors = true;
		ErrorMessage = FText::Format(LOCTEXT("UserCommentTooLongError", "Description may only be a maximum of {0} characters (currently {1})"), SizeLimit, Size);
	}

	CrashDetailsInformation->SetError(ErrorMessage);
}

EVisibility SCrashReportClient::IsHintTextVisible() const
{
	return CrashDetailsInformation->GetText().IsEmpty() ? EVisibility::HitTestInvisible : EVisibility::Hidden;
}

bool SCrashReportClient::IsSendEnabled() const
{
	const bool bValidAppName = FPrimaryCrashProperties::Get()->IsValid() && !FPrimaryCrashProperties::Get()->GameName.IsEmpty();
	const bool bValidEndPoint = !FCrashReportCoreConfig::Get().GetReceiverAddress().IsEmpty() || !FCrashReportCoreConfig::Get().GetDataRouterURL().IsEmpty();
	const bool bIsAllowedToSendWithoutDetailedInfo = FCrashReportCoreConfig::Get().IsAllowedToSendWithoutDetailedInfo();

	const bool bHasValidInput = bValidAppName && bValidEndPoint && !bHasUserCommentErrors;
	const bool bCanSend = bIsAllowedToSendWithoutDetailedInfo || !bIsUserCommentEmpty;
	return bHasValidInput && bCanSend;
}

FText SCrashReportClient::GetSendTooltip()
{
	// Optionally show a tooltip with the endpoint domain to the user. If the old receiver address is used it is
	// just a IP number, so there is no point is showing that.
	static FText CachedDomain = []() {
		FStringView Endpoint = FCrashReportCoreConfig::Get().GetReceiverAddress();
		bool bIsUrl = false;
		if (Endpoint.IsEmpty())
		{
			Endpoint = FCrashReportCoreConfig::Get().GetDataRouterURL();
			bIsUrl = true;
		}
		if (Endpoint.IsEmpty())
		{
			return LOCTEXT("SendTooltipEmpty", "No server specified.");
		}
		if (bIsUrl && FCrashReportCoreConfig::Get().GetShowEndpointInTooltip())
		{
			// Show only domain, not full url
			const int32 Start = Endpoint.StartsWith(TEXT("https://")) ? 8 : 0;
			Endpoint.RightChopInline(Start);
			int32 End(INDEX_NONE);
			Endpoint.FindChar('/', End);
			Endpoint.LeftInline(End);
			return FText::Format(LOCTEXT("SendTooltipUrl", "Send to {0}"), FText::FromStringView(Endpoint));
		}
		return LOCTEXT("SendTooltip", "Send to server");
	}();
	return CachedDomain;
}

FText SCrashReportClient::GetContactText()
{
	static FText CachedContactText = []()
	{
		const FStringView Company = FCrashReportCoreConfig::Get().GetCompanyName();
		if (Company.IsEmpty())
		{
			return LOCTEXT("IAgreeNoCompany", "I agree to be contacted via email if additional information about this crash would help fix it.");
		}
		return FText::Format(
			 LOCTEXT("IAgreeCompany", "I agree to be contacted by {0} via email if additional information about this crash would help fix it."),
			 FText::FromStringView(Company)
		);
	}();
	return CachedContactText;
}

#undef LOCTEXT_NAMESPACE

#endif // !CRASH_REPORT_UNATTENDED_ONLY
