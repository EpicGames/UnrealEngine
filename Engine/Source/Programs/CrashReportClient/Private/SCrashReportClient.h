// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CrashReportClient.h"

#if !CRASH_REPORT_UNATTENDED_ONLY

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

/**
 * UI for the crash report client app
 */
class SCrashReportClient : public SCompoundWidget
{
public:
	/**
	 * Slate arguments
	 */
	SLATE_BEGIN_ARGS(SCrashReportClient)
		: _bHideSubmitAndRestart(false)
	{
	}

	/** Should the Submit and Send button be hitten. This can be overriden by a platform settings in the crash report config ini file. */
	SLATE_ARGUMENT(bool, bHideSubmitAndRestart)

	SLATE_END_ARGS()

	/**
	 * Construct this Slate ui
	 * @param InArgs Slate arguments, not used
	 * @param Client Crash report client implementation object
	 * @param bSimpleDialog Whether to use the simple dialog UI that implicitly sends the report
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FCrashReportClient>& Client, bool bSimpleDialog);

	bool IsFinished() { return CrashReportClient->IsUploadComplete() && CrashReportClient->ShouldWindowBeHidden(); }

private:
	/**
	 * Construct the detailed Slate ui with controls for sending the report and comments
	 * @param Client Crash report client implementation object
	 */
	void ConstructDetailedDialog(const TSharedRef<FCrashReportClient>& Client, const FText& CrashDetailedMessage);

	/**
	 * Construct the minimal Slate ui with just a button to close
	 * @param Client Crash report client implementation object
	 */
	void ConstructSimpleDialog(const TSharedRef<FCrashReportClient>& Client, const FText& CrashDetailedMessage);

	/**
	 * Keyboard short-cut handler
	 * @param InKeyEvent Which key was released, and which auxiliary keys were pressed
	 * @return Whether the event was handled
	 */
	FReply OnUnhandledKeyDown(const FKeyEvent& InKeyEvent);

	/** Called if the multi line widget text changes */
	void OnUserCommentTextChanged(const FText& NewText);

	/** Whether the hint text should be visible. */
	EVisibility IsHintTextVisible() const;

	/** Whether the send buttons are enabled. */
	bool IsSendEnabled() const;

	/** Returns the tooltop text for send button */
	static FText GetSendTooltip();

	/** Returns the 'allow contact' text */
	static FText GetContactText();

#if PLATFORM_WINDOWS
	/** Whether the copy to clipboard button is available. */
	bool IsCopyToClipboardEnabled() const;
#endif

	/** Crash report client implementation object */
	TSharedPtr<FCrashReportClient> CrashReportClient;

	TSharedPtr<SMultiLineEditableTextBox> CrashDetailsInformation;

	bool bIsUserCommentEmpty;
	bool bHasUserCommentErrors;
	bool bHideSubmitAndRestart;

	bool bIncludeScreenshotInCrashReport = false;
};

#endif // !CRASH_REPORT_UNATTENDED_ONLY
