// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportClientApp.h"
#include "Windows/WindowsHWrapper.h"
#include "CrashReportClientDefines.h"

#if CRASH_REPORT_WITH_MTBF
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformStackWalk.h"
#include "Serialization/Archive.h"
#include "CrashReportAnalyticsSessionSummary.h"
#endif

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"

void CopyDiagnosticFilesToClipboard(TConstArrayView<FString> Files)
{
	if( OpenClipboard(GetActiveWindow()) )
	{
		verify(EmptyClipboard());
		HGLOBAL GlobalMem;
		SIZE_T RequiredSize = sizeof(DROPFILES) + sizeof(TCHAR);
		for (const FString& File : Files)
		{
			RequiredSize += (File.Len() * sizeof(TCHAR)) + sizeof(TCHAR);
		}
		GlobalMem = GlobalAlloc( GMEM_MOVEABLE, RequiredSize );
		check(GlobalMem);
		uint8* Data = (uint8*) GlobalLock( GlobalMem );
		check(Data);
		DROPFILES* Drop = (DROPFILES*)Data;
		Drop->pFiles = sizeof(DROPFILES);
		Drop->fWide = 1;
		TCHAR* Dest = (TCHAR*)(Data + sizeof(DROPFILES));
		TCHAR* End = (TCHAR*)(Data + RequiredSize);
		for (const FString& File : Files)
		{
			FCString::Strncpy(Dest, *File, End - Dest);	
			Dest += (File.Len() + 1);
		}
		GlobalUnlock( GlobalMem );
		if( SetClipboardData( CF_HDROP, GlobalMem ) == NULL )
		{
			UE_LOGF(LogWindows, Fatal,"SetClipboardData failed with error code %i", (uint32)GetLastError() );
		}

		verify(CloseClipboard());
	}
	else
	{
		UE_LOGF(LogWindows, Warning, "OpenClipboard failed with error code %i", (uint32)GetLastError());
	}
}

#if CRASH_REPORT_WITH_MTBF && !PLATFORM_SEH_EXCEPTIONS_DISABLED

static ANSICHAR CrashStackTrace[8*1024] = {0};

void SaveCrcCrashException(EXCEPTION_POINTERS* ExceptionInfo)
{
	// Try to write the exception code in the appropriate field if the session was created. The first crashing thread
	// incrementing the counter wins the race and can write its exception code.
	static volatile int32 CrashCount = 0;
	if (FPlatformAtomics::InterlockedIncrement(&CrashCount) == 1)
	{
		FCrashReportAnalyticsSessionSummary::Get().OnCrcCrashing(ExceptionInfo->ExceptionRecord->ExceptionCode);

		if (ExceptionInfo->ExceptionRecord->ExceptionCode != STATUS_HEAP_CORRUPTION)
		{
			// Try to get the exception callstack to log to figure out why CRC crashed. This is not robust because this runs
			// in the crashing processs and it allocates memory/use callstack, but we may still be able to get some useful data.
			if (FPlatformStackWalk::InitStackWalkingForProcess(FProcHandle()))
			{
				FPlatformStackWalk::StackWalkAndDump(CrashStackTrace, UE_ARRAY_COUNT(CrashStackTrace), 0);
				if (CrashStackTrace[0] != 0)
				{
					FCrashReportAnalyticsSessionSummary::Get().LogEvent(ANSI_TO_TCHAR(CrashStackTrace));
				}
			}
		}
	}
}

/**
 * The Vectored Exception Handler (VEH) is added to capture heap corruption exceptions because those are not reaching the
 * UnhandledExceptionFilter(). VEH has first and only chance to heap corrutpion exceptions before they got 'handled' by the OS.
 */
LONG WINAPI CrashReportVectoredExceptionFilter(EXCEPTION_POINTERS* ExceptionInfo)
{
	if (ExceptionInfo->ExceptionRecord->ExceptionCode == STATUS_HEAP_CORRUPTION)
	{
		SaveCrcCrashException(ExceptionInfo);
	}

	// Let the OS deal with the exception. (the process will crash)
	return EXCEPTION_CONTINUE_SEARCH;
}

/**
 * Invoked when an exception was not handled by vectored exception handler(s) nor structured exception handler(s)(__try/__except).
 * For good understanding a SEH inner working,take a look at EngineUnhandledExceptionFilter documentation in WindowsPlatformCrashContext.cpp.
 */
LONG WINAPI CrashReportUnhandledExceptionFilter(EXCEPTION_POINTERS* ExceptionInfo)
{
	SaveCrcCrashException(ExceptionInfo);

	// Let the OS deal with the exception. (the process will crash)
	return EXCEPTION_CONTINUE_SEARCH;
}
#endif

/**
 * WinMain, called when the application is started
 */
int WINAPI WinMain(_In_ HINSTANCE hInInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPSTR, _In_ int nCmdShow)
{
	hInstance = hInInstance;

#if CRASH_REPORT_WITH_MTBF // For the Editor only.
	FString Arguments(::GetCommandLineW());
	if (Arguments.Contains(TEXT("-MONITOR=")) && !Arguments.Contains(TEXT("-RespawnedInstance")))
	{
		uint64 ChildPipe = 0;
		FParse::Value(GetCommandLineW(), TEXT("-READ="), ChildPipe);

		// Parse the process ID of the Editor that spawned this CRC.
		uint32 MonitoredEditorPid = 0;
		if (FParse::Value(GetCommandLineW(), TEXT("-MONITOR="), MonitoredEditorPid))
		{
			TCHAR RespawnExePathname[MAX_PATH];
			GetModuleFileName(NULL, RespawnExePathname, MAX_PATH);
			FString RespawnExeArguments(Arguments);
			RespawnExeArguments.Append(" -RespawnedInstance");
			UE::HAL::EProcessId RespawnPid;

			// Respawn itself to sever the process grouping with the Editor. If the user kills the Editor process group in task manager,
			// CRC will not die at the same time, will be able to capture the Editor exit code and send the MTBF report to correctly
			// identify the Editor 'AbnormalShutdown' as 'Killed' instead.
			UE::HAL::FProcess Process
			{
				{
					.Uri = RespawnExePathname,
					.Arguments = *RespawnExeArguments,
					.bDetached = true,
					.bInheritHandles = true
				},
				RespawnPid
			};

			if (Process)
			{
				FString PidPathname = FString::Printf(TEXT("%sue-crc-pid-%d"), FPlatformProcess::UserTempDir(), MonitoredEditorPid);
				if (TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*PidPathname, FILEWRITE_EvenIfReadOnly)))
				{
					std::underlying_type_t<UE::HAL::EProcessId> Pid = std::underlying_type_t<UE::HAL::EProcessId>(RespawnPid);
					*Ar << Pid;
				}
			}
		}
		RequestEngineExit(TEXT("Respawn instance."));
		return 0; // Exit this intermediate instance, the Editor is waiting for it to continue.
	}

#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	::SetUnhandledExceptionFilter(CrashReportUnhandledExceptionFilter);
	::AddVectoredExceptionHandler(0, CrashReportVectoredExceptionFilter);
#endif // !PLATFORM_SEH_EXCEPTIONS_DISABLED
#endif // CRASH_REPORT_WITH_MTBF

	RunCrashReportClient(GetCommandLineW());
	return 0;
}
