// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Pre-compiled header includes
#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "CrashReportCoreConfig.h"

#if !CRASH_REPORT_UNATTENDED_ONLY
	#include "StandaloneRenderer.h"
#endif // CRASH_REPORT_UNATTENDED_ONLY

/**
 * Run the crash report client app
 */
void RunCrashReportClient(const TCHAR* Commandline);
